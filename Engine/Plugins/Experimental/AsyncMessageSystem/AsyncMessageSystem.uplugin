{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"Description": "Async Message System",
	"Category": "Framework",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "AsyncMessageSystem",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	]
}