// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AsyncMessageSystem : ModuleRules
	{
		public AsyncMessageSystem(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			// Enable truncation warnings in this plugin
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
			
			// Uncomment this line to make for easier debugging
			//OptimizeCode = CodeOptimization.Never;
			
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"DeveloperSettings",
					"Engine",
					"GameplayTags",
				}
			);
		}
	}
}