// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncAction_ListenForMessage.h"

#include "AsyncGameplayMessageSystem.h"
#include "AsyncMessageBindingComponent.h"
#include "AsyncMessageWorldSubsystem.h"
#include "AsyncMessageSystemLogs.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncAction_ListenForMessage)

UAsyncAction_ListenForAsyncMessage* UAsyncAction_ListenForAsyncMessage::StartListeningForAsyncMessage(
	UObject* WorldContextObject,
	const FAsyncMessageId MessageId,
	TScriptInterface<IAsyncMessageBindingEndpointInterface> DesiredEndpoint,
	const TEnumAsByte<ETickingGroup> TickGroup)
{
	check(GEngine);
	
	// Use the GEngine version of GetWorld instead of a simpler UObject function call because it has nice
	// log/error raising for BP users.
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!World)
	{
		return nullptr;
	}

	// Create a new Async BP action to listen for a message
	UAsyncAction_ListenForAsyncMessage* Action = NewObject<UAsyncAction_ListenForAsyncMessage>();
	Action->WeakWorldPtr = World;
	Action->DesiredEndpoint = DesiredEndpoint ? DesiredEndpoint->GetEndpoint() : nullptr;
	Action->MessageToListenFor = MessageId;
	Action->BindingOptions.SetTickGroup(TickGroup);
	
	Action->RegisterWithGameInstance(World);
	
	return Action;
}

void UAsyncAction_ListenForAsyncMessage::StopListeningForAsyncMessage()
{
	// Mark this async action as being no long needed, which will unbind our listeners from the message system
	SetReadyToDestroy();
}

void UAsyncAction_ListenForAsyncMessage::Activate()
{
	Super::Activate();

	StartListeningForMessage();
}

void UAsyncAction_ListenForAsyncMessage::SetReadyToDestroy()
{
	Super::SetReadyToDestroy();

	UnbindListener();
}

void UAsyncAction_ListenForAsyncMessage::StartListeningForMessage()
{
	check (!BoundListenerHandle.IsValid());
	
	if (!MessageToListenFor.IsValid())
	{
		UE_LOGF(LogAsyncMessageSystem, Error, "[%s] Invalid MessageToListenFor '%ls' on action %ls", __func__, *MessageToListenFor.ToString(), *GetNameSafe(this));
		return;
	}

	TSharedPtr<FAsyncGameplayMessageSystem> Sys = GetAssociatedMessageSystem();
	if (!Sys.IsValid())
	{
		UE_LOGF(LogAsyncMessageSystem, Error, "[%s] Failed to find the associated message system for async action %ls", __func__, *GetNameSafe(this));
		return;
	}

	// Bind the listener for this message
	BoundListenerHandle = Sys->BindListener(
		MessageToListenFor,
		TWeakObjectPtr<UAsyncAction_ListenForAsyncMessage>{ this },
		&UAsyncAction_ListenForAsyncMessage::HandleMessageReceived,
		BindingOptions,
		DesiredEndpoint);

	// If for some reason we failed to bind the handle, then destroy this async action and log an error
	if (!BoundListenerHandle.IsValid())
	{
		UE_LOGF(LogAsyncMessageSystem, Error, "[%s] Failed to bind listener for async action %ls", __func__, *GetNameSafe(this));
		
		SetReadyToDestroy();
		return;
	}

	// Listen for when the subsystem starts to shut down so that we can unbind ourselves and clean up properly
	UAsyncMessageWorldSubsystem* WorldSubsystem = GetAssociatedSubsystem();
	if (!WorldSubsystem)
	{
		UE_LOGF(LogAsyncMessageSystem, Error, "[%s] Failed to find async message world subsystem %ls", __func__, *GetNameSafe(this));
		SetReadyToDestroy();
		return;
	}
	
	check(!OnMessageSystemShutdownDelegateHandle.IsValid());
	OnMessageSystemShutdownDelegateHandle = WorldSubsystem->OnShutdownMessageSystem.AddWeakLambda(this, [this]()
	{
		// Mark ourselves as being ready for destruction when the associated message subsystem goes out of scope
		SetReadyToDestroy();
	});
}

void UAsyncAction_ListenForAsyncMessage::UnbindListener()
{
	// Stop listening to the subsystem shutdown delegate
	if (OnMessageSystemShutdownDelegateHandle.IsValid())
	{
		if (TObjectPtr<UAsyncMessageWorldSubsystem> WorldSubsys = GetAssociatedSubsystem())
		{
			WorldSubsys->OnShutdownMessageSystem.Remove(OnMessageSystemShutdownDelegateHandle);
		}
	}

	// Stop listening for the actual message callback
	if (!BoundListenerHandle.IsValid())
	{
		return;
	}
	
	TSharedPtr<FAsyncGameplayMessageSystem> Sys = GetAssociatedMessageSystem();
	if (!Sys.IsValid())
	{
		return;
	}

	// Unbind this async action listener
	Sys->UnbindListener(BoundListenerHandle);
}

void UAsyncAction_ListenForAsyncMessage::HandleMessageReceived(const FAsyncMessage& Message)
{
	ensure(BoundListenerHandle.IsValid());
	
	OnMessageReceived.Broadcast(Message);
}

TSharedPtr<FAsyncGameplayMessageSystem> UAsyncAction_ListenForAsyncMessage::GetAssociatedMessageSystem() const
{
	TStrongObjectPtr<UWorld> World = WeakWorldPtr.Pin();
	if (!World.IsValid())
	{
		return nullptr;
	}

	return UAsyncMessageWorldSubsystem::GetSharedMessageSystem<FAsyncGameplayMessageSystem>(World.Get());
}

TObjectPtr<UAsyncMessageWorldSubsystem> UAsyncAction_ListenForAsyncMessage::GetAssociatedSubsystem() const
{
	TStrongObjectPtr<UWorld> World = WeakWorldPtr.Pin();
	if (!World.IsValid())
	{
		return nullptr;
	}

	return World->GetSubsystem<UAsyncMessageWorldSubsystem>();
}
