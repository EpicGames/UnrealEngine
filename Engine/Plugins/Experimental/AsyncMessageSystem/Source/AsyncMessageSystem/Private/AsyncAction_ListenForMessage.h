// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AsyncMessageBindingOptions.h"
#include "AsyncMessageHandle.h"
#include "AsyncMessageId.h"
#include "AsyncMessageBindingComponent.h"
#include "Kismet/BlueprintAsyncActionBase.h"
#include "StructUtils/InstancedStruct.h"

#include "AsyncAction_ListenForMessage.generated.h"

struct FAsyncMessage;
class FAsyncGameplayMessageSystem;
class UAsyncMessageWorldSubsystem;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FAsyncMessageReceivedDelegate, const FAsyncMessage&, Message);

/**
 * An async action for binding a listener to a message. The user will call the
 * "StartListeningForAsyncMessage" function from blueprints, which will create the async action object
 * and an exec pin will be created for any BP Assignable delegates on this object (FAsyncMessageReceivedDelegate).
 *
 * Those delegates can then fire when the message is received, and we can allow users to specify their own tick groups upon binding as well.
 * We expose this as an async proxy (ExposedAsyncProxy) because then you can easily call the "StopListeningForAsyncMessage" function
 * to unbind the listener.
 *
 * If for some reason there is a failure when binding to the message, this async task will be immedately marked as being ready for destruction.
 */
UCLASS(BlueprintType, meta=(ExposedAsyncProxy = "AsyncMessageTask"))
class UAsyncAction_ListenForAsyncMessage : public UBlueprintAsyncActionBase
{
	GENERATED_BODY()
	
public:
	
	/**
	 * Starts listening for an Async Message with the given ID during the given tick group. 
	 * 
	 * @param MessageId The ID of the Async Message which you would like to listen to
	 * @param DesiredEndpoint The endpoint which this listener should bind to. If nothing is provided, the default world endpoint will be used.
	 * @param TickGroup The tick group which you would like to receive the message in. Default is TG_PostUpdateWork.
	 */
	UFUNCTION(BlueprintCallable, Category = "Async Messages", meta=(WorldContext="WorldContextObject", BlueprintInternalUseOnly="true", AdvancedDisplay="TickGroup", DisplayName="Start Listening for Async Message"))
	static UAsyncAction_ListenForAsyncMessage* StartListeningForAsyncMessage(
		UObject* WorldContextObject,
		const FAsyncMessageId MessageId,
		TScriptInterface<IAsyncMessageBindingEndpointInterface> DesiredEndpoint,
		const TEnumAsByte<ETickingGroup> TickGroup = TG_PostUpdateWork);

	/**
	 * Stops the given message handle from receiving any more messages.
	 */
	UFUNCTION(BlueprintCallable, Category = "Async Messages")
	void StopListeningForAsyncMessage();

	/**
	 * Delegate which executes when the bound message that this async action is listening for is broadcast.
	 */
	UPROPERTY(BlueprintAssignable)
	FAsyncMessageReceivedDelegate OnMessageReceived;
	
protected:

	//~ Begin UBlueprintAsyncActionBase interface
	virtual void Activate() override;
	virtual void SetReadyToDestroy() override;
	//~ End of UBlueprintAsyncActionBase interface

	/**
	 * Binds a listener to for the MessageToListenFor message id to the HandleMessageReceived function 
	 */
	void StartListeningForMessage();

	/**
	 * Unbinds the listener handle for this async action so no more messages will be processed by it.
	 */
	void UnbindListener();

	/**
	 * The callback function which will call the OnMessageReceived delegate and give blueprints
	 * the oppurtunity to process the message.
	 */
	void HandleMessageReceived(const FAsyncMessage& Message);

	/**
	 * @return The gameplay message system associated with the outer world of this async action.
	 */
	TSharedPtr<FAsyncGameplayMessageSystem> GetAssociatedMessageSystem() const;

	/**
	 * @return The async message world subsystem associated with the outer world of this async action. 
	 */
	TObjectPtr<UAsyncMessageWorldSubsystem> GetAssociatedSubsystem() const;
	
	/**
	* Weak pointer to the owning wold context to which this listener belongs 
	*/
	TWeakObjectPtr<UWorld> WeakWorldPtr = nullptr;

	/**
	 * The specific endpoint to start listening for this message on.
	 */
	TWeakPtr<FAsyncMessageBindingEndpoint> DesiredEndpoint = nullptr;
	
	/**
	* The async message id that this action should listen 
	*/
	FAsyncMessageId MessageToListenFor = FAsyncMessageId::Invalid;

	/**
	 * The binding options to use when listening for this message.
	 * 
	 * The tick group of this binding option is set on construction of this object in StartListeningForAsyncMessage.
	 */
	FAsyncMessageBindingOptions BindingOptions = {};

	/**
	 * The listener handle which has been bound to the message 
	 */
	FAsyncMessageHandle BoundListenerHandle = FAsyncMessageHandle::Invalid;

	/**
	 * Handle to a delegate which fires when the world subsystem which owns the message system
	 * that this async action is listening for shuts down. This allows us to clean up this async
	 * action and mark it as being ready for destruction.
	 */
	FDelegateHandle OnMessageSystemShutdownDelegateHandle;
};