// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncGameplayMessageSystem.h"

#include "Async/Async.h"
#include "AsyncMessageSystemLogs.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "HAL/IConsoleManager.h"
#include "Tasks/TaskPrivate.h"

namespace UE::Private
{
	static FString LexToString(const ETickingGroup Group)
	{
		static FString Invalid = TEXT("Invalid");
		static UEnum* TGEnum = StaticEnum<ETickingGroup>();
		
		return TGEnum ? TGEnum->GetDisplayNameTextByValue(Group).ToString() : Invalid;
	};

	static bool bUseTaskSyncManager = false;
	static FAutoConsoleVariableRef CVarUseTaskSyncManager(TEXT("AsyncMessageSystem.UseTaskSyncManager"),
		bUseTaskSyncManager,
		TEXT("If true, the TaskSyncManager will be used to control when tick group messages will be processed."));

	/** The description to use when creating sync points with the task sync manager */
	static const FName AsyncMessageSyncPointDesc = TEXT("AsyncGameplayMessageSystem_SyncPointDesc");

	static const TMap<ETickingGroup, FName> SyncPointTickGroupNames =
	{
		{ TG_PrePhysics, 		"TG_PrePhysics" 	},
		{ TG_StartPhysics, 		"TG_StartPhysics" 	},
		{ TG_DuringPhysics, 	"TG_DuringPhysics"	},
		{ TG_EndPhysics, 		"TG_EndPhysics" 	},
		{ TG_PostPhysics, 		"TG_PostPhysics" 	},
		{ TG_PostUpdateWork, 	"TG_PostUpdateWork" },
	};
}

/**
 * Tick function which will being the processing of messages for specific tick groups on the message system. 
 */
struct FMessageSystemTickFunction final : public FTickFunction
{
	friend class FAsyncGameplayMessageSystem;
	
	explicit FMessageSystemTickFunction(const ETickingGroup InGroup, TWeakPtr<FAsyncGameplayMessageSystem> InWeakMessageSystem)
		: FTickFunction()
		, WeakMessageSys(InWeakMessageSystem)
	{
		bCanEverTick = true;
		bStartWithTickEnabled = !UE::Private::bUseTaskSyncManager;
		bAllowTickBatching = true;

		// Only run this tick function on the game thread, because our message system is our "sync point"
		// for everything. 
		bRunOnAnyThread = false;

		// We want to ensure that we start and end in the same tick group to make sure we are a valid sync point for other threads
		TickGroup = InGroup;
		EndTickGroup = TickGroup;
	}

	// FTickFunction interface
	virtual void ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override
	{
		// Each tick function can simply call the message system and let it know that the next tick group has started.
		if (TSharedPtr<FAsyncGameplayMessageSystem> MessageSys = WeakMessageSys.Pin())
		{
			MessageSys->ProcessMessagesForTickGroup(TickGroup);
		}
	}
	
	virtual FString DiagnosticMessage() override
	{
		return TEXT("FMessageSystemTickFunction::") + UE::Private::LexToString(TickGroup);
	}

	/**
	 * The owning message system which this tick function is going to update
	 */
	TWeakPtr<FAsyncGameplayMessageSystem> WeakMessageSys = nullptr;
};

FAsyncGameplayMessageSystem::FAsyncGameplayMessageSystem(UWorld* OwningWorld)
	: OuterWorld(MakeWeakObjectPtr<UWorld>(OwningWorld))
{
	
}

void FAsyncGameplayMessageSystem::Startup_Impl()
{
	check(OuterWorld.IsValid());
	
	// Create a tick function for each tick group
	CreateTickFunctions();
}

void FAsyncGameplayMessageSystem::Shutdown_Impl()
{
	// Remove all tick groups and wait for them to finish
	DestroyTickFunctions();
}

void FAsyncGameplayMessageSystem::PostQueueMessage(const FAsyncMessageId MessageId, const TArray<FAsyncMessageBindingOptions>& OptionsBoundTo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMessageSystem::PostQueueMessage);

	// When we queue a message, check if there are any listeners outside of tick groups who would need a specific
	// async task to process their message queue. 
	for (const FAsyncMessageBindingOptions& BindingOpts : OptionsBoundTo)
	{
		switch (BindingOpts.GetType())
		{
		case FAsyncMessageBindingOptions::EBindingType::UseTickGroup:
			EnableExecutionForTickGroupBinding(BindingOpts);
			break;
		case FAsyncMessageBindingOptions::EBindingType::UseNamedThreads:
		case FAsyncMessageBindingOptions::EBindingType::UseTaskPriorities:
			StartAsyncProcessForBinding(BindingOpts);
			break;
		}
	}
}

void FAsyncGameplayMessageSystem::CreateTickFunctions()
{
	check(TickFunctions.IsEmpty());

	if (!ensureMsgf(OuterWorld.IsValid(), TEXT("Failed to create message system tick functions, the outer world is invalid!")))
	{
		return;
	}

	FScopeLock WorkHandleLock(&WorkHandleCreationCS);

	// We will be binding to some owning world's persistent level to create our tick functions
	ULevel* TickLevel = OuterWorld->PersistentLevel;

	TWeakPtr<FAsyncGameplayMessageSystem> WeakThisPtr = StaticCastWeakPtr<FAsyncGameplayMessageSystem>(this->AsWeak());

	// Track the previous tick function so that we can add it as a dependency for each to finish
	TSharedPtr<FTickFunction> PreviousTickFunction = nullptr;

	const int StartingGroup = static_cast<int>(EarliestSupportedTickGroup);
	const int LastGroup = static_cast<int>(LatestSupportedTickGroup);

	using namespace UE::Tick;
	FTaskSyncManager* SyncMan = UE::Private::bUseTaskSyncManager ? FTaskSyncManager::Get() : nullptr;

	// Spawn a tick function for every tick group that we can actually do any work in
	for (int i = StartingGroup; i <= LastGroup; ++i)
	{
		const ETickingGroup Group = static_cast<ETickingGroup>(i);

		TSharedPtr<FTickFunction> Func = MakeShared<FMessageSystemTickFunction>(Group, WeakThisPtr);

		if (!SyncMan)
		{
			// Fall back to the "normal" tick functoin process of ticking every frame
			// if we are not using the TaskSyncManager
			Func->RegisterTickFunction(TickLevel);

			// We always want the previous tick function to wait until the next one to start processing
			if (PreviousTickFunction.IsValid())
			{
				Func->AddPrerequisite(TickLevel, *PreviousTickFunction);
			}	
		}
		else
		{
			const FName* GroupSyncPointName = UE::Private::SyncPointTickGroupNames.Find(Group);
			check(GroupSyncPointName);
			
			FSyncPointDescription Desc = {};
			Desc.FirstPossibleTickGroup = Group;
			Desc.LastPossibleTickGroup = Group;
			Desc.EventType = ESyncPointEventType::GameThreadTask;
			Desc.ActivationRules = ESyncPointActivationRules::WaitForAllWork;
			Desc.SourceName = UE::Private::AsyncMessageSyncPointDesc;
			Desc.RegisteredName = *GroupSyncPointName;
			
			SyncMan->RegisterNewSyncPoint(Desc);
		}

		PreviousTickFunction = Func;

		// Keep track of the tick functions we have created so that we can properly unregister them later
		TickFunctions.Add(Group, MoveTemp(Func));
	}
}

void FAsyncGameplayMessageSystem::DestroyTickFunctions()
{
	using namespace UE::Tick;

	FScopeLock WorkHandleLock(&WorkHandleCreationCS);
	
	FTaskSyncManager* SyncMan = UE::Private::bUseTaskSyncManager ? FTaskSyncManager::Get() : nullptr;
	
	for (TPair<ETickingGroup, TSharedPtr<FTickFunction>>& Pair : TickFunctions)
	{
		if (Pair.Value.IsValid())
		{
			Pair.Value->UnRegisterTickFunction();
		}
		
		if (SyncMan)
		{
			const FName* GroupSyncPointName = UE::Private::SyncPointTickGroupNames.Find(Pair.Key);
			check(GroupSyncPointName);
			
			SyncMan->UnregisterSyncPoint(*GroupSyncPointName, UE::Private::AsyncMessageSyncPointDesc);
		}
	}

	TickFunctions.Empty();

	// The work handle will abandon work if necessary here in its destructor, so we can just empty the map.
	WorkHandles.Empty();
}

void FAsyncGameplayMessageSystem::StartAsyncProcessForBinding(const FAsyncMessageBindingOptions& Options)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMessageSystem::StartAsyncProcessForBinding);

	// We only want to do this for task ID's and priorities. Tick groups are already being processed via our tick functions
	check(
		Options.GetType() == FAsyncMessageBindingOptions::EBindingType::UseNamedThreads ||
		Options.GetType() == FAsyncMessageBindingOptions::EBindingType::UseTaskPriorities);

	// Kick off a weak lambda to process the message queue for this task graph ID, ensuring
	// that any listeners bound to these options will get the message called back when they expect
	TWeakPtr<FAsyncGameplayMessageSystem> WeakThisPtr = StaticCastWeakPtr<FAsyncGameplayMessageSystem>(this->AsWeak());
	
	UE::Tasks::ETaskPriority TaskPri = Options.GetTaskPriority();
	UE::Tasks::EExtendedTaskPriority ExtendedTaskPri = Options.GetExtendedTaskPriority();

	// Translate from the old named thread model to the newer UE Tasks model if we need to
	if (Options.GetType() == FAsyncMessageBindingOptions::EBindingType::UseNamedThreads)
	{
		const ENamedThreads::Type ThreadToProcessOn = Options.GetNamedThreads();
		UE::Tasks::Private::TranslatePriority(ThreadToProcessOn, OUT TaskPri, OUT ExtendedTaskPri);
	}
	
	UE::Tasks::Launch(UE_SOURCE_LOCATION, [WeakThisPtr, Options]()
    {
    	if (TSharedPtr<FAsyncGameplayMessageSystem> This = WeakThisPtr.Pin())
    	{
    		This->ProcessMessageQueueForBinding(Options);
    	}
    },
    TaskPri,
    ExtendedTaskPri);
}

void FAsyncGameplayMessageSystem::EnableExecutionForTickGroupBinding(const FAsyncMessageBindingOptions& Options)
{
	check(Options.GetType() == FAsyncMessageBindingOptions::EBindingType::UseTickGroup);

	if (!UE::Private::bUseTaskSyncManager)
	{
		return;
	}

	FScopeLock WorkHandleLock(&WorkHandleCreationCS);

	const ETickingGroup Group = Options.GetTickGroup();
	
	UE::Tick::FActiveSyncWorkHandle* WorkHandle = GetOrCreateWorkHandleForTickGroup(Group);
	if (!WorkHandle)
	{
		UE_LOGF(LogAsyncMessageSystem, Error, "Failed to get a work handle for group '%ls'. Messages will not be processed.",
			*UE::Private::LexToString(Options.GetTickGroup()));
		return;
	}

	TSharedPtr<FTickFunction>* TickFunc = TickFunctions.Find(Group);
	if (!ensure(TickFunc))
	{
		UE_LOGF(LogAsyncMessageSystem, Error, "Failed to get a tick function '%ls'. Messages will not be processed.",
			*UE::Private::LexToString(Options.GetTickGroup()));
		return;
	}
	
	const bool bSuccess = WorkHandle->RequestWork(TickFunc->Get(), UE::Tick::ESyncWorkRepetition::Once);

	UE_CLOGF(!bSuccess, LogAsyncMessageSystem, Error, "Failed to request work for group '%ls'. Messages will not be processed.",
		*UE::Private::LexToString(Options.GetTickGroup()));
}

UE::Tick::FActiveSyncWorkHandle* FAsyncGameplayMessageSystem::GetOrCreateWorkHandleForTickGroup(const ETickingGroup Group)
{
	if (!ensure(UE::Private::bUseTaskSyncManager))
	{
		return nullptr;
	}
	
	//FScopeLock WorkHandleLock(&WorkHandleCreationCS);

	// First, check if we have a handle already.
	if (UE::Tick::FActiveSyncWorkHandle* ExistingHandle = WorkHandles.Find(Group))
	{
		return ExistingHandle;
	}

	UWorld* World = OuterWorld.Get();
	if (!World)
	{
		UE_LOGF(LogAsyncMessageSystem, Error, "Invalid world, cannot create work handles.");
		return nullptr;
	}
	
	using namespace UE::Tick;
	FTaskSyncManager* SyncMan = FTaskSyncManager::Get();
	if (!SyncMan)
	{
		UE_LOGF(LogAsyncMessageSystem, Error, "Unable to get a valid FTaskSyncManager. Messages may not be processed.");
		return nullptr;
	}
	const FName* GroupSyncPointName = UE::Private::SyncPointTickGroupNames.Find(Group);
	check(GroupSyncPointName);
	
	FSyncPointId SyncPoint = SyncMan->FindSyncPoint(World, *GroupSyncPointName);
	if (!SyncPoint.IsValid())
	{
		UE_LOGF(LogAsyncMessageSystem, Error, "Failed to get a sync point for group '%ls'. Messages will not be processed.",
			*UE::Private::LexToString(Group));
		return nullptr;
	}

	FActiveSyncWorkHandle WorkHandle;
	FTaskSyncResult Res = SyncMan->RegisterWorkHandle(SyncPoint, WorkHandle);
	if (!ensure(Res.WasSuccessful()))
	{
		UE_LOGF(LogAsyncMessageSystem, Error, "Failed to register work handle for group '%ls'. Messages will not be processed.",
			*UE::Private::LexToString(Group));
	}

	return &WorkHandles.Add(Group, MoveTemp(WorkHandle));
}

void FAsyncGameplayMessageSystem::ProcessMessagesForTickGroup(const ETickingGroup TickGroup)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMessageSystem::ProcessMessagesForTickGroup);

	if (!OuterWorld.IsValid())
	{
		UE_LOGF(LogAsyncMessageSystem, Warning, "[%s] OuterWorld weak pointer is no longer valid for message system. Messages will not be processed, and this system will be shut down.", __func__);
		Shutdown();
		return;
	}
	
	CurrentTickGroup = TickGroup;

	ensure(CurrentTickGroup >= EarliestSupportedTickGroup || CurrentTickGroup <= LatestSupportedTickGroup);	

	// Process the messages for this current tick group
	FAsyncMessageBindingOptions Options = {};
	Options.SetTickGroup(CurrentTickGroup);
	
	ProcessMessageQueueForBinding(Options);
		
	LastTickedGroup = TickGroup;
}
