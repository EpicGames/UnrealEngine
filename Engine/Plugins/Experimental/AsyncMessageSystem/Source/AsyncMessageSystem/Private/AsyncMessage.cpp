// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncMessage.h"
#include "AsyncMessageBindingEndpoint.h"

#if ENABLE_ASYNC_MESSAGES_DEBUG
#include "HAL/PlatformStackWalk.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncMessage)

FAsyncMessage::FAsyncMessage(
	const FAsyncMessageId& InMessageId,
	const FAsyncMessageId& InMessageSourceId,
	const double MessageTimestamp,
	const uint64 CurrentFrame,
	const uint32 InThreadQueuedFrom,
	const uint32 InMessageSequenceCount,
	FConstStructView InPayloadData,
	TWeakPtr<FAsyncMessageBindingEndpoint> InBindingEndpoint)
	: MessageId(InMessageId)
	, MessageSourceId(InMessageSourceId)
	, QueueTime(MessageTimestamp)
	, QueueFrame(CurrentFrame)
	, ThreadQueuedFrom(InThreadQueuedFrom)
	, SequenceId(InMessageSequenceCount)
	, PayloadCopy(InPayloadData)	// <-- This is making a copy of the payload data!!! $$$
									// It calls the explicit FInstancedStruct(const FConstStructView InOther) constructor
									// which is going to copy the UScriptStruct.
									// This is the same as : 
									//		PayloadCopy.InitializeAs(InPayloadData.GetScriptStruct(), InPayloadData.GetMemory());
									// TODO: Use some custom linear allocator here on the message system to make the copying cheaper
	, BindingEndpoint(InBindingEndpoint)
{

}

double FAsyncMessage::GetQueueTimestamp() const
{
	return QueueTime;
}

uint64 FAsyncMessage::GetQueueFrame() const
{
	return QueueFrame;
}

uint32 FAsyncMessage::GetThreadQueuedFromThreadId() const
{
	return ThreadQueuedFrom;
}

uint32 FAsyncMessage::GetSequenceId() const
{
	return SequenceId;
}

FAsyncMessageId FAsyncMessage::GetMessageId() const
{
	return MessageId;
}

void FAsyncMessage::SetMessageId(const FAsyncMessageId& NewMessageId)
{
	MessageId = NewMessageId;
}

FAsyncMessageId FAsyncMessage::GetMessageSourceId() const
{
	return MessageSourceId;
}

FStructView FAsyncMessage::GetPayloadView()
{
	return FStructView(PayloadCopy);
}

FConstStructView FAsyncMessage::GetPayloadView() const
{
	return FConstStructView(PayloadCopy);
}

TSharedPtr<FAsyncMessageBindingEndpoint> FAsyncMessage::GetBindingEndpoint() const
{
	return BindingEndpoint.Pin();
}

#if ENABLE_ASYNC_MESSAGES_DEBUG

const FString& FAsyncMessage::FMessageDebugData::GetOrCreateNativeCallstackAsString() const
{
	if (NativeCallstackAsString.Len() > 0)
	{
		return NativeCallstackAsString;
	}

	// Decode the callstack and cache it in a string
	for (int32 i = 0; i < NativeCallstack.Num(); i++)
	{
		FProgramCounterSymbolInfo LineInfo;
		FPlatformStackWalk::ProgramCounterToSymbolInfo(NativeCallstack[i], LineInfo);

		const ANSICHAR* FilenameWithoutPath = nullptr;
		if ((FilenameWithoutPath = FCStringAnsi::Strrchr(LineInfo.Filename, '/')) == nullptr)
		{
			if ((FilenameWithoutPath = FCStringAnsi::Strrchr(LineInfo.Filename, '\\')) == nullptr)
			{
				FilenameWithoutPath = LineInfo.Filename;
			}
		}

		ANSICHAR LineString[MAX_SPRINTF];
		FCStringAnsi::Sprintf(LineString, "%-64s (%s:%i)\n", LineInfo.FunctionName, FilenameWithoutPath+1, LineInfo.LineNumber);

		NativeCallstackAsString.Append(LineString);
	}

	return NativeCallstackAsString;
}

const FString& FAsyncMessage::GetNativeCallstack() const
{
	static const FString Unknown = TEXT("Unknown");
	return DebugData ? DebugData->GetOrCreateNativeCallstackAsString() : Unknown;
}

const FString& FAsyncMessage::GetBlueprintScriptCallstack() const
{
	static const FString Unknown = TEXT("Unknown");
	return DebugData ? DebugData->BlueprintScriptCallstack : Unknown;
}

const uint32 FAsyncMessage::GetDebugMessageId() const
{
	return DebugData ? DebugData->MessageId : 0u;
}

void FAsyncMessage::SetDebugData(FMessageDebugData* InData)
{
	DebugData = InData;
}
#endif	// #if ENABLE_ASYNC_MESSAGES_DEBUG
