// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncMessageBindingComponent.h"
#include "AsyncMessageBindingEndpoint.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncMessageBindingComponent)

void UAsyncMessageBindingComponent::InitializeComponent()
{
	Super::InitializeComponent();

	CreateEndpoint();
}

void UAsyncMessageBindingComponent::UninitializeComponent()
{
	Super::UninitializeComponent();

	CleanupEndpoint();
}

TSharedPtr<FAsyncMessageBindingEndpoint> UAsyncMessageBindingComponent::GetEndpoint() const
{
	return Endpoint;
}

void UAsyncMessageBindingComponent::CreateEndpoint()
{
	// If there is already a valid endpoint, then there is no need to do anything.
	if (Endpoint.IsValid())
	{
		return;
	}
	
	Endpoint = MakeShared<FAsyncMessageBindingEndpoint>();
}

void UAsyncMessageBindingComponent::CleanupEndpoint()
{
	Endpoint.Reset();
}
