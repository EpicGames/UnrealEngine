// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncMessageBindingEndpoint.h"

FAsyncMessageBindingEndpoint::FAsyncMessageBoundData* FAsyncMessageBindingEndpoint::GetBoundDataForMessage(const FAsyncMessageId& MessageId)
{
	return BoundMessageListenerMap.Find(MessageId);
}

FAsyncMessageBindingEndpoint::FAsyncMessageBoundData& FAsyncMessageBindingEndpoint::FindOrAddMessageData(const FAsyncMessageId& MessageId)
{
	return BoundMessageListenerMap.FindOrAdd(MessageId);
}

uint32 FAsyncMessageBindingEndpoint::GetNumberOfBoundListeners() const
{
	uint32 Res = 0;
	
	for (const TPair<FAsyncMessageId, FAsyncMessageBindingEndpoint::FAsyncMessageBoundData>& MessagePair : BoundMessageListenerMap)
	{
		const FAsyncMessageBindingEndpoint::FAsyncMessageBoundData& BoundData = MessagePair.Value;
		for (const TPair<FAsyncMessageBindingOptions, TArray<FAsyncMessageBindingEndpoint::FAsyncMessageIndividualListener<FMessageCallbackFunc>>>& ListenerPair : BoundData.ListenerMap)
		{
			Res += ListenerPair.Value.Num();
		}
	}	

	return Res;
}

bool FAsyncMessageBindingEndpoint::IsHandleBound(const FAsyncMessageHandle& Handle) const
{
	if (const FAsyncMessageBoundData* FoundBindingData = BoundMessageListenerMap.Find(Handle.GetBoundMessageId()))
	{
		for (const TPair<FAsyncMessageBindingOptions, TArray<FAsyncMessageIndividualListener<FMessageCallbackFunc>>>& ListenerPair : FoundBindingData->ListenerMap)
		{
			const bool bHasHandle = ListenerPair.Value.ContainsByPredicate([Handle](const FAsyncMessageIndividualListener<FMessageCallbackFunc>& Listener)
			{
				return Listener.Handle == Handle;	
			});
	
			if (bHasHandle)
			{
				return true;
			}
		}	
	}
	
	return false;
}
