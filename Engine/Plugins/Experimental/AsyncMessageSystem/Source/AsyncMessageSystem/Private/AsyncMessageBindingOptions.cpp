// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncMessageBindingOptions.h"

FAsyncMessageBindingOptions::FAsyncMessageBindingOptions()
{
	// Default to after main ticks
	SetTickGroup(TG_PostUpdateWork);
}

FAsyncMessageBindingOptions::FAsyncMessageBindingOptions(const ETickingGroup DesiredTickGroup)
{ 
	SetTickGroup(DesiredTickGroup);
}

FAsyncMessageBindingOptions::FAsyncMessageBindingOptions(const ENamedThreads::Type NamedThreads)
{
	SetNamedThreads(NamedThreads);
}

FAsyncMessageBindingOptions::FAsyncMessageBindingOptions(const UE::Tasks::ETaskPriority InTaskPriority, const UE::Tasks::EExtendedTaskPriority InExtendedTaskPriority)
{
	SetTaskPriorities(InTaskPriority, InExtendedTaskPriority);
}

void FAsyncMessageBindingOptions::SetTickGroup(const ETickingGroup DesiredTickGroup)
{
	Type = EBindingType::UseTickGroup;
	ThreadOrGroup = static_cast<int32>(DesiredTickGroup);
}

ETickingGroup FAsyncMessageBindingOptions::GetTickGroup() const
{
	return (Type == EBindingType::UseTickGroup) ? static_cast<ETickingGroup>(ThreadOrGroup) : ETickingGroup::TG_MAX;
}

void FAsyncMessageBindingOptions::SetNamedThreads(const ENamedThreads::Type NamedThreads)
{
	Type = EBindingType::UseNamedThreads;
	ThreadOrGroup = static_cast<int32>(NamedThreads);
}

ENamedThreads::Type FAsyncMessageBindingOptions::GetNamedThreads() const
{
	return (Type == EBindingType::UseNamedThreads) ? static_cast<ENamedThreads::Type>(ThreadOrGroup) : ENamedThreads::UnusedAnchor;
}

void FAsyncMessageBindingOptions::SetTaskPriorities(const UE::Tasks::ETaskPriority InTaskPriority, const UE::Tasks::EExtendedTaskPriority InExtendedTaskPriority)
{
	Type = EBindingType::UseTaskPriorities;
	TaskPriority = InTaskPriority;
	ExtendedTaskPriority = InExtendedTaskPriority;
}

UE::Tasks::ETaskPriority FAsyncMessageBindingOptions::GetTaskPriority() const
{
	return (Type == EBindingType::UseTaskPriorities) ? TaskPriority : UE::Tasks::ETaskPriority::Default;
}

UE::Tasks::EExtendedTaskPriority FAsyncMessageBindingOptions::GetExtendedTaskPriority() const
{
	return (Type == EBindingType::UseTaskPriorities) ? ExtendedTaskPriority : UE::Tasks::EExtendedTaskPriority::None;
}