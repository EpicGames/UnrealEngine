// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncMessageBlueprintLibrary.h"

#include "AsyncMessageWorldSubsystem.h"
#include "AsyncMessageSystemBase.h"
#include "AsyncMessageSystemLogs.h"
#include "Engine/Engine.h"	// For GEngine 

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncMessageBlueprintLibrary)

bool UAsyncMessageSystemBlueprintLibrary::QueueAsyncMessageForBroadcast(
	UObject* WorldContextObject,
	const FAsyncMessageId& MessageId,
	const FInstancedStruct& Payload,
	TScriptInterface<IAsyncMessageBindingEndpointInterface> DesiredEndpoint)
{
	check(GEngine);
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!World)
	{
		UE_LOGF(LogAsyncMessageSystem, Error, "[%s] Failed to queue message '%ls' for broadcasting: Unable to find a world.",
			__func__, *MessageId.ToString());
		
		return false;
	}

	TSharedPtr<FAsyncMessageSystemBase> Sys = UAsyncMessageWorldSubsystem::GetSharedMessageSystem(World);
	if (!Sys.IsValid())
	{
		UE_LOGF(LogAsyncMessageSystem, Error, "[%s] Failed to queue message '%ls' for broadcasting: Unable to find a message system for world '%ls'.",
			__func__, *MessageId.ToString(),  *GetNameSafe(World));
		
		return false;
	}

	TWeakPtr<FAsyncMessageBindingEndpoint> WeakEndpoint = DesiredEndpoint ? DesiredEndpoint->GetEndpoint() : nullptr;

	return Sys->QueueMessageForBroadcast(MessageId, Payload, WeakEndpoint);
}

FString UAsyncMessageSystemBlueprintLibrary::Conv_AsyncMessageIdToString(const FAsyncMessageId& MessageId)
{
	return MessageId.ToString();
}

FString UAsyncMessageSystemBlueprintLibrary::GetMessageNativeQueueCallstack(const FAsyncMessage& Message)
{
#if ENABLE_ASYNC_MESSAGES_DEBUG
	return Message.GetNativeCallstack();
#else
	return TEXT("Unknown: ENABLE_ASYNC_MESSAGES_DEBUG is disabled in this build configuration");
#endif	// #if ENABLE_ASYNC_MESSAGES_DEBUG
}

FString UAsyncMessageSystemBlueprintLibrary::GetMessageBlueprintScriptCallstack(const FAsyncMessage& Message)
{
#if ENABLE_ASYNC_MESSAGES_DEBUG
	return Message.GetBlueprintScriptCallstack();
#else
	return TEXT("Unknown: ENABLE_ASYNC_MESSAGES_DEBUG is disabled in this build configuration");
#endif	// #if ENABLE_ASYNC_MESSAGES_DEBUG
}
