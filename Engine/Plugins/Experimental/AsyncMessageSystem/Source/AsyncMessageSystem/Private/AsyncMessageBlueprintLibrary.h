// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AsyncMessageId.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "AsyncMessageBindingComponent.h"

#include "AsyncMessageBlueprintLibrary.generated.h"

/**
 * Blueprint function library for the Async Message System
 */
UCLASS()
class UAsyncMessageSystemBlueprintLibrary final : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Queues the given async message for broadcast the next time that this
	 * message system processes its message queue
	 *
	 * @param MessageId		The message that you would like to queue for broadcasting
	 * @param Payload		The payload data of this message. This payload data will be COPIED to the message queue to
	 *						to make it safe for listeners on other threads.
	 * @param DesiredEndpoint The endpoint which this listener should bind to. If nothing is provided, the default world endpoint will be used. 
	 *
	 * @return True if this message had an listeners bound to it and it was successfully queued
	 */
	UFUNCTION(BlueprintCallable, Category = "Async Messages", meta=(WorldContext="WorldContextObject", AutoCreateRefTerm = "Payload" ,ReturnDisplayName = "Success"))
	static bool QueueAsyncMessageForBroadcast(
		UObject* WorldContextObject,
		const FAsyncMessageId& MessageId,
		const FInstancedStruct& Payload,
		TScriptInterface<IAsyncMessageBindingEndpointInterface> DesiredEndpoint);

	/**
	 * Get the string representation of the given Async Message Id.
	 * 
	 * @param MessageId The Message Id to get the string representation of
	 * @return The String representation of the give async message Id
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|String", meta=(DisplayName = "To String (Async Message Id)", CompactNodeTitle = "->", BlueprintAutocast))
	static FString Conv_AsyncMessageIdToString(const FAsyncMessageId& MessageId);

	/**
	 * Gets the given message's callstack of when and where it was queued from native C++ code.
	 *
	 * Note: bShouldRecordQueueCallstackOnMessages must be enabled in the project settings for this to have accurate data
	 * 
	 * @param Message The async message to get the callstack of to determine when and where it was queued.
	 * @return The callstack of where the message was queued from
	 */
	UFUNCTION(BlueprintPure, Category = "Async Messages|Debug")
	static FString GetMessageNativeQueueCallstack(const FAsyncMessage& Message);

	/**
	 * Gets the given message's callstack of when and where it was queued from in blueprints/script
	 *
	 * Note: bShouldRecordQueueCallstackOnMessages must be enabled in the project settings for this to have accurate data
	 * 
	 * @param Message The async message to get the callstack of to determine when and where it was queued.
	 * @return The script callstack of where the message was queued from
	 */
	UFUNCTION(BlueprintPure, Category = "Async Messages|Debug")
	static FString GetMessageBlueprintScriptCallstack(const FAsyncMessage& Message);
	
};
