// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncMessageDeveloperSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncMessageDeveloperSettings)

bool UAsyncMessageDeveloperSettings::GetShouldEnableWorldSubsystem() const
{
	return bEnableWorldSubsystem;
}

bool UAsyncMessageDeveloperSettings::GetShouldEnableWorldSubsystemInEditor() const
{
	return bEnableWorldSubsystemInEditor;
}

bool UAsyncMessageDeveloperSettings::ShouldDebugMessageOnQueue(const FAsyncMessageId& Message) const
{
	if (!bMessageQueueDebugEnabled)
	{
		return false;
	}

	if (bEnabledDebuggingForAllQueuedMessages)
	{
		return true;
	}

	return EnabledDebugMessages.Contains(Message);
}

bool UAsyncMessageDeveloperSettings::ShouldTriggerBreakPointOnMessageQueue() const
{
	return bTriggerDebugBreakpointWhenMessageQueued;
}

bool UAsyncMessageDeveloperSettings::ShouldPrintScriptCallstackOnMessageQueue() const
{
	return bPrintScriptCallstackWhenMessageQueued;
}

bool UAsyncMessageDeveloperSettings::ShouldRecordQueueCallstackOnMessages() const
{
	return bShouldRecordQueueCallstackOnMessages;
}
