// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AsyncMessageId.h"
#include "Engine/DeveloperSettings.h"

#include "AsyncMessageDeveloperSettings.generated.h"

UCLASS(Config=Game)
class UAsyncMessageDeveloperSettings final : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	
	[[nodiscard]] bool GetShouldEnableWorldSubsystem() const;
	[[nodiscard]] bool GetShouldEnableWorldSubsystemInEditor() const;

	/**
	 * @param Message The message to check if it should have its stack trace dumped to the log 
	 * @return True if this message should dump its current stack trace to the log
	 */
	[[nodiscard]] bool ShouldDebugMessageOnQueue(const FAsyncMessageId& Message) const;

	/**
	 * @return True if a breakpoint should be triggered when a message id with debugging enabled is queued for broadcast. 
	 */
	[[nodiscard]] bool ShouldTriggerBreakPointOnMessageQueue() const;

	/**
	 * @return True if the Blueprint VM callstack should be printed when a message with debugging enabled is queued for broadcast.
	 */
	[[nodiscard]] bool ShouldPrintScriptCallstackOnMessageQueue() const;

	/**
	 * @return True if the callstack should be recorded as a property on the Async Message itself. This
	 * can make debugging where a message was queued from signifigantly easier if you are starting from the
	 * listener. 
	 */
	[[nodiscard]] bool ShouldRecordQueueCallstackOnMessages() const;

private:
	
	/**
	 * Enabled the conditional logging of stack traces and C++ breakpoints being triggered when certain
	 * messages are queued for broadcasting to the message system.
	 *
	 * This can make it easier to debug where messages are coming from, since the processing of
	 * messages is all deferred and it can be difficult to track down what is queuing a message in
	 * cooked/optimized builds.
	 *
	 * This will only work if the ENABLE_ASYNC_MESSAGES_DEBUG pre-processor definition is non-zero.
	 * By default, ENABLE_ASYNC_MESSAGES_DEBUG is "1" for any non-shipping or editor build (WITH_EDITOR || !UE_BUILD_SHIPPING).
	 * 
	 * If you wish to override this behavior, then you can add a definition to your game's .build.cs file:
	 *		PublicDefinitions.Add("ENABLE_ASYNC_MESSAGES_DEBUG=1");
	 *
	 * @see AsyncMessageSystemBase.h
	 * @see FAsyncMessageSystemBase::QueueMessageForBroadcast
	 */
	UPROPERTY(EditDefaultsOnly, Config, Category = "Async Messages|Debug")
	bool bMessageQueueDebugEnabled = false;

	/**
	 * If true, then a native C++ breakpoint will be triggered when when certain
	 * messages are queued for broadcasting to the message system.
	 * 
	 * @see FAsyncMessageSystemBase::QueueMessageForBroadcast
	 */
	UPROPERTY(EditDefaultsOnly, Config, Category = "Async Messages|Debug", meta=(EditCondition="bMessageQueueDebugEnabled"))
	bool bTriggerDebugBreakpointWhenMessageQueued = false;

	/**
	 * If true, then the script callstack will also be printed when certain
	 * messages are queued for broadcasting to the message system.
	 *
	 * @see FAsyncMessageSystemBase::QueueMessageForBroadcast
	 */
	UPROPERTY(EditDefaultsOnly, Config, Category = "Async Messages|Debug", meta=(EditCondition="bMessageQueueDebugEnabled"))
	bool bPrintScriptCallstackWhenMessageQueued = false;

	/**
	 * If true, then the callstack at the time of a message being queued will be recorded and stored on the
	 * FAsyncMessage instance itself. This can make debugging listeners significantly easier if you
	 * need to get an idea of where a message is coming from.
	 *
	 * @see FAsyncMessageSystemBase::QueueMessageForBroadcast_Impl
	 */
	UPROPERTY(EditDefaultsOnly, Config, Category = "Async Messages|Debug")
	bool bShouldRecordQueueCallstackOnMessages = false;

	/**
	 * If true, then ALL messages queued for broadcast will have their debug information processed.
	 * 
	 * Note: This will likely have a large performance impact.
	 *
	 * If false, then only messages in the "EnabledDebugMessages" array will be debugged when queued.
	 */
	UPROPERTY(EditDefaultsOnly, Config, Category = "Async Messages|Debug", meta=(EditCondition="bMessageQueueDebugEnabled"))
	bool bEnabledDebuggingForAllQueuedMessages = false;
	
	/**
	* Array of Async Message Id's which you would like to enable for debugging 
	* when they are queued for broadcasting to the message system.
	*/
	UPROPERTY(EditDefaultsOnly, Config, Category = "Async Messages|Debug", meta=(EditCondition="bMessageQueueDebugEnabled && !bEnabledDebuggingForAllQueuedMessages"))
	TArray<FAsyncMessageId> EnabledDebugMessages;
	
	/**
	 * If true, then the async message world subsystem will be enabled.
	 */
	UPROPERTY(EditDefaultsOnly, Config, Category = "Async Messages")
	bool bEnableWorldSubsystem = true;

	/**
	 * If true, then the async message subsystem will be created for editor worlds.
	 */
	UPROPERTY(EditDefaultsOnly, Config, Category = "Async Messages")
	bool bEnableWorldSubsystemInEditor = false;
};