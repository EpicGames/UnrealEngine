// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncMessageHandle.h"
#include "AsyncMessageBindingEndpoint.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncMessageHandle)

const FAsyncMessageHandle FAsyncMessageHandle::Invalid = FAsyncMessageHandle();

bool FAsyncMessageHandle::IsValid() const
{
	return InternalHandle != FAsyncMessageHandle::InvalidHandleIndex;
}

uint32 FAsyncMessageHandle::GetId() const
{
	return InternalHandle;
}

FAsyncMessageId FAsyncMessageHandle::GetBoundMessageId() const
{
	return BoundMessage;
}

FString FAsyncMessageHandle::ToString() const
{
	return FString::Printf(TEXT("%u"), InternalHandle);
}

bool FAsyncMessageHandle::operator==(const FAsyncMessageHandle& Other) const
{
	return InternalHandle == Other.InternalHandle && BindingEndpoint == Other.BindingEndpoint; 
}

bool FAsyncMessageHandle::operator!=(const FAsyncMessageHandle& Other) const
{
	return !(*this == Other); 
}

bool FAsyncMessageHandle::operator>=(const FAsyncMessageHandle& Other) const
{
	return InternalHandle >= Other.InternalHandle;
}

bool FAsyncMessageHandle::operator<(const FAsyncMessageHandle& Other) const
{
	return InternalHandle < Other.InternalHandle;
}

TSharedPtr<FAsyncMessageBindingEndpoint> FAsyncMessageHandle::GetBindingEndpoint() const
{
	return BindingEndpoint.Pin();
}

uint32 GetTypeHash(const FAsyncMessageHandle& InMapping)
{
	return GetTypeHash(InMapping.InternalHandle);
}

//////////////////////////////////////////////////////////
// Internal creation of message handles...
FAsyncMessageHandle::FAsyncMessageHandle(
	 const uint32 InHandleValue,
	 const FAsyncMessageId InBoundMessage,
	 TWeakPtr<FAsyncMessageBindingEndpoint> InBindingEndpoint)
	: InternalHandle(InHandleValue)
	, BoundMessage(InBoundMessage)
	, BindingEndpoint(InBindingEndpoint) 
{
	checkf(InHandleValue != InvalidHandleIndex, TEXT("'%ud' is an invalid value for FAsyncMessageHandle!"), InHandleValue);
	checkf(BoundMessage.IsValid(), TEXT("'%s' is an invalid FAsyncMessageId to create a handle for!"), *BoundMessage.ToString());
	checkf(BindingEndpoint.IsValid(), TEXT("Message handle for message '%s' does not have a valid handler!"), *BoundMessage.ToString());
}
