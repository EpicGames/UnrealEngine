// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncMessageId.h"
#include "GameplayTagContainer.h"
#include "GameplayTagsManager.h"

// A message with an empty gameplay tag is considered invalid

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncMessageId)
const FAsyncMessageId FAsyncMessageId::Invalid = {};

FAsyncMessageId::FAsyncMessageId(const FName MessageName)
	: InternalMessageTag(FGameplayTag::RequestGameplayTag(MessageName))
{
}

FAsyncMessageId::FAsyncMessageId(const FGameplayTag& MessageTag)
	: InternalMessageTag(MessageTag)
{
}

bool FAsyncMessageId::IsValid() const
{
	return InternalMessageTag.IsValid();
}

FName FAsyncMessageId::GetMessageName() const
{
	return InternalMessageTag.GetTagName();
}

FString FAsyncMessageId::ToString() const
{
	return InternalMessageTag.ToString();
}

FAsyncMessageId FAsyncMessageId::GetParentMessageId() const
{
	return FAsyncMessageId(InternalMessageTag.RequestDirectParent());
}

void FAsyncMessageId::WalkMessageHierarchy(const FAsyncMessageId StartingMessage, TFunctionRef<void(const FAsyncMessageId MessageId)> ForEachMessageFunc)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncMessageId::WalkMessageHierarchy);

	if (!ensure(StartingMessage.IsValid()))
	{
		return;
	}
	
	// Base tag first
	ForEachMessageFunc(StartingMessage);

	TArray<FGameplayTag> ParentTags;
	UGameplayTagsManager::Get().ExtractParentTags(StartingMessage.InternalMessageTag, ParentTags);

	// This will be the first parent up to the root tag
	for (const FGameplayTag& Parent : ParentTags)
	{
		ForEachMessageFunc(FAsyncMessageId(Parent));
	}
}
