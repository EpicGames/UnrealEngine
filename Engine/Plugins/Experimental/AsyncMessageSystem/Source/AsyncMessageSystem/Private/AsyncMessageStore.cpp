// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncMessageStore.h"

#if ENABLE_ASYNC_MESSAGES_DEBUG

#include "AsyncMessageDeveloperSettings.h"
#include "HAL/PlatformStackWalk.h"
#include "Stats/StatsMisc.h"

uint32 FAsyncMessageStore::CreateDebugMessageData()
{
	const uint32 DebugDataId = NextDebugMessageId++;
	TTuple<uint32, FAsyncMessage::FMessageDebugData>& DebugData = MessageDebugCallstacks.Add(DebugDataId, TTuple<uint32, FAsyncMessage::FMessageDebugData>{ });

	DebugData.Value.MessageId = DebugDataId;
	
	// Stack trace. Do this fast and don't bother getting the symbols (tanks performance)
	constexpr int32 StackTraceDepth = 32;
	uint64 StackTrace[StackTraceDepth];
	const uint32 StackDepth = FPlatformStackWalk::CaptureStackBackTrace(StackTrace, StackTraceDepth);
	const uint32 NumStackItemsToSkip = 6;
	
	if (StackDepth > NumStackItemsToSkip)
	{
		DebugData.Value.NativeCallstack = TArray<uint64>(&StackTrace[NumStackItemsToSkip], StackDepth - NumStackItemsToSkip);
	}

	// Also record the BP script callstack in case this was queued from a blueprint/script call
	DebugData.Value.BlueprintScriptCallstack = FFrame::GetScriptCallstack();
	
	return DebugDataId;
}
#endif	// #if ENABLE_ASYNC_MESSAGES_DEBUG

uint32 FAsyncMessageStore::EnqueueMessage(FAsyncMessage&& MessageToQueue, OUT TArray<FAsyncMessageBindingOptions>& BindingTypesAddedTo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncMessageStore::EnqueueMessage);
	
	// We want to keep track of which BindingOptions had listeners for this message ID
	BindingTypesAddedTo.Reset();
	
	FScopeLock QueueLock(&MessageQueueCriticalSection);
	
	uint32 NumQueuesAddedTo = 0u;

#if ENABLE_ASYNC_MESSAGES_DEBUG
	const bool bRecordQueueCallstack = GetDefault<UAsyncMessageDeveloperSettings>()->ShouldRecordQueueCallstackOnMessages();
	uint32 DebugDataId = 0u;
#endif	// #if ENABLE_ASYNC_MESSAGES_DEBUG

	// Check if there are any listeners to this event or any of its parents. If there are, add it to their binding queue
	FAsyncMessageId::WalkMessageHierarchy(MessageToQueue.GetMessageId(), [&](const FAsyncMessageId& CurrentMessageId)
	{
		// Queue this message to each of the bound options
		for (TPair<FAsyncMessageBindingOptions, FBindingOptionsMessageQueue>& Pair : MessageBindingQueues)
		{
			// Only add this message to a queue if we know that it has a listener for this particular message.
			if (Pair.Value.MessagesWithTheseBindings.Contains(CurrentMessageId) && !BindingTypesAddedTo.Contains(Pair.Key))
			{
				const int32 AddedMessageIdx = Pair.Value.MessageQueue->Add(MessageToQueue);
				++NumQueuesAddedTo;
			
				BindingTypesAddedTo.Emplace(Pair.Key);

#if ENABLE_ASYNC_MESSAGES_DEBUG
				if (bRecordQueueCallstack)
				{
					// Find or create debug data here and set a pointer to it on each message
					if (DebugDataId == 0u)
					{
						DebugDataId = CreateDebugMessageData();
					}

					TTuple<uint32, FAsyncMessage::FMessageDebugData>& DebugData = MessageDebugCallstacks[DebugDataId];
					++DebugData.Key;
					check(DebugData.Key == NumQueuesAddedTo);
					
					(*Pair.Value.MessageQueue)[AddedMessageIdx].SetDebugData(&DebugData.Value);
				}
#endif	// #if ENABLE_ASYNC_MESSAGES_DEBUG
			}
		}	
	});
	
	return NumQueuesAddedTo;
}

TOptional<FAsyncMessage> FAsyncMessageStore::GetNextMessageForBindingOption(const FAsyncMessageBindingOptions& Binding)
{
	// TODO: We can probably have one big lock for this instead of locking on access internally like this
	FScopeLock QueueLock(&MessageQueueCriticalSection);	

	if (FBindingOptionsMessageQueue* QueueContainer = MessageBindingQueues.Find(Binding))
	{
		// If we can, then pop an element off the message queue
		if (!QueueContainer->MessageQueue->IsEmpty())
		{
#if ENABLE_ASYNC_MESSAGES_DEBUG
			TOptional<FAsyncMessage> OutVal{ QueueContainer->MessageQueue->Pop(EAllowShrinking::No) };

			// Remove the debug data from the map if the number of queues that the message was initially added to
			// have all been processed
			if (OutVal.IsSet())
			{
				const uint32 DebugMessageId = (*OutVal).GetDebugMessageId();
				if (TTuple<uint32, FAsyncMessage::FMessageDebugData>* Data = MessageDebugCallstacks.Find(DebugMessageId))
				{
					--Data->Key;
					if (Data->Key == 0)
					{
						MessageDebugCallstacks.Remove(DebugMessageId);
					}
				}
			}
			
			return OutVal;
#else
			return QueueContainer->MessageQueue->Pop(EAllowShrinking::No);
#endif
		}
	}
	
	return TOptional<FAsyncMessage>{};
}

void FAsyncMessageStore::AddMessageToBinding(const FAsyncMessageId& MessageId, const FAsyncMessageBindingOptions& Binding)
{
	FScopeLock QueueLock(&MessageQueueCriticalSection);
	
	FBindingOptionsMessageQueue& QueueData = MessageBindingQueues.FindOrAdd(Binding);

	// Keep track of how many listeners are bound to this message with these binding options
	uint32& Count = QueueData.MessagesWithTheseBindings.FindOrAdd(MessageId, /* default value */ 0u);
	++Count;
}

bool FAsyncMessageStore::RemoveMessageFromBinding(const FAsyncMessageId& MessageId, const FAsyncMessageBindingOptions& Binding)
{
	FScopeLock QueueLock(&MessageQueueCriticalSection);

	// If we have no messages that match this binding, then there is nothing to be done
	FBindingOptionsMessageQueue* QueueData = MessageBindingQueues.Find(Binding);
	if (!QueueData)
	{
		return false;
	}

	// Decrement the count of how many messages of this ID are on this binding option
	if (uint32* Count = QueueData->MessagesWithTheseBindings.Find(MessageId))
	{
		(*Count)--;

		// If there are no more, then we can remove this message ID from the data store
		if ((*Count) == 0u)
		{
			// Return true if successfully removed
			return QueueData->MessagesWithTheseBindings.Remove(MessageId) > 0;
		}
	}

	// Otherwise there are still other references to this message bound to this binding, so don't remove it
	return false;
}

void FAsyncMessageStore::AddReferencedObjects(FReferenceCollector& Collector)
{
	FScopeLock QueueLock(&MessageQueueCriticalSection);

	// For each message in the queue, add any GC refs that its payload may have.
	for (TPair<FAsyncMessageBindingOptions, FBindingOptionsMessageQueue>& Pair : MessageBindingQueues)
	{
		FBindingOptionsMessageQueue& QueueData = Pair.Value;
		
		for (FAsyncMessage& Message : *(QueueData.MessageQueue.Get()))
		{
			Message.AddReferencedObjects(Collector);
		}
	}
}