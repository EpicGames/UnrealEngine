// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncMessageSystemBase.h"

#include "AsyncMessageDeveloperSettings.h"
#include "AsyncMessageBindingEndpoint.h"
#include "AsyncMessageSystemLogs.h"
#include "CoreGlobals.h"
#include "Misc/App.h"
#include "NativeGameplayTags.h"

#if ENABLE_ASYNC_MESSAGES_DEBUG
#include "Misc/AssertionMacros.h"
#endif

FAsyncMessageSystemBase::~FAsyncMessageSystemBase()
{
	// Make sure that you call the "Shutdown" function on your message system to allow it to clean up before destruction!
	ensureMsgf(bIsShuttingDown == true, TEXT("A message system was destructed but did not have its shutdown function called"));
}

FAsyncMessageHandle FAsyncMessageSystemBase::BindListener(
	const FAsyncMessageId MessageId,
	FMessageCallbackFunc&& Callback,
	const FAsyncMessageBindingOptions& Options /* = {} */,
	TWeakPtr<FAsyncMessageBindingEndpoint> BindingEndpoint /* = nullptr */)
{
	const FAsyncMessageHandle OutHandle = GenerateNextValidMessageHandle(MessageId, BindingEndpoint);
	
	const bool bWasSuccessfullyBound = BindListener_Impl(
		OutHandle,
		MessageId,
		MoveTemp(Callback),
		Options);
	
	return bWasSuccessfullyBound ? OutHandle : FAsyncMessageHandle::Invalid;
}

void FAsyncMessageSystemBase::UnbindListener(const FAsyncMessageHandle& HandleToUnbind)
{
	UnbindListener_Impl(HandleToUnbind);
}

bool FAsyncMessageSystemBase::QueueMessageForBroadcast(
	const FAsyncMessageId MessageId,
	FConstStructView PayloadData,
	TWeakPtr<FAsyncMessageBindingEndpoint> BindingEndpoint /*= nullptr*/)
{
#if ENABLE_ASYNC_MESSAGES_DEBUG
	const UAsyncMessageDeveloperSettings* Settings = GetDefault<UAsyncMessageDeveloperSettings>();
	
	// If this message is configured to have its stack trace dumped when queued, then do so
	if (Settings && Settings->ShouldDebugMessageOnQueue(MessageId))
	{
		const uint32 ThreadQueuedFrom = FPlatformTLS::GetCurrentThreadId();
		const FString Heading = FString::Printf(TEXT("=== Message '%s' queued from thread %u ==="), *MessageId.ToString(), ThreadQueuedFrom);
		
		FDebug::DumpStackTraceToLog(*Heading, ELogVerbosity::Display);

		// Print the BP callstack if desired
		if (Settings->ShouldPrintScriptCallstackOnMessageQueue())
		{
			PrintScriptCallstack();
		}
		
		// Note: In case you forgot to enable the printing of the Blueprint VM callstack, you can use these commands
		// in the "immediate" window to print it instead:
		// 
		// In editor builds:
		//		{,,UnrealEditor-Core}::PrintScriptCallstack()
		// In a game/client build:
		//		::PrintScriptCallstack()

		// Also allow for users to easily configure breakpointers when these messages are queued, making it easy
		// to track down where and why the messages are being broadcast.
		if (Settings->ShouldTriggerBreakPointOnMessageQueue())
		{
			UE_DEBUG_BREAK();	
		}
	}
#endif	// ENABLE_ASYNC_MESSAGES_DEBUG

	TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncMessageSystemBase::QueueMessageForBroadcast);
	
	// If BindingEndpoint is not set, then use the default listen handler.
	// We will require a default listen handler to be set on the message system before you can queue any messages
	if (!DefaultBindingEndpoint.IsValid())
	{
		UE_LOGF(LogAsyncMessageSystem, Error, "[%s] Failed to queue message: There is no valid DefaultBindingEndpoint on this message system.", __func__);
		return false;
	}
	
	return QueueMessageForBroadcast_Impl(
		MessageId,
		MoveTemp(PayloadData),
		BindingEndpoint.IsValid() ? BindingEndpoint : DefaultBindingEndpoint);
}

void FAsyncMessageSystemBase::ProcessMessageQueueForBinding(const FAsyncMessageBindingOptions& Options)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncMessageSystemBase::ProcessMessageQueueForBinding);

	// Handle any pre-processing which may need to happen before sending messages. This could include
	// binding listeners, removing old handles, and more.
	PreProcessMessagesQueue(Options);

	// Actually process the messages
	ProcessMessageQueueForBinding_Impl(Options);
}

void FAsyncMessageSystemBase::Startup()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncMessageSystemBase::Startup);
	
	UE_LOGF(LogAsyncMessageSystem, Verbose, "[%s] Message System Startup...", __func__);
	
	checkf(bIsShuttingDown == false && DefaultBindingEndpoint.IsValid() == false, TEXT("Attempting to restart a message system is not supported."));

	// Create a default handler
	DefaultBindingEndpoint = MakeShared<FAsyncMessageBindingEndpoint>();
	
	Startup_Impl();
}

void FAsyncMessageSystemBase::Shutdown()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncMessageSystemBase::Shutdown);
	
	UE_LOGF(LogAsyncMessageSystem, Verbose, "[%s] Message System Shutdown...", __func__);
	
	// Flag this message system as being shut down so that we don't
	// attempt to queue any more messages or start any new tasks
	bIsShuttingDown = true;

	DefaultBindingEndpoint.Reset();

	Shutdown_Impl();
}

void FAsyncMessageSystemBase::AddReferencedObjects(UObject* InReferencer, FReferenceCollector& Collector)
{
	// Add references from the message store 
	MessageStore.AddReferencedObjects(Collector);

	// Note: We are explictly NOT adding references to the _listener_ objects in the BoundMessageListenerMap
	// because the message system only allows you to bind to weak references. We do not want a message system
	// to keep a refernce to a listener object alive in GC, and we handle their removal in the BindListener
	// lambda functions already.
}

///////////////////////////////////////////////////////////
// Virtual Implementations

bool FAsyncMessageSystemBase::BindListener_Impl(
	const FAsyncMessageHandle& HandleToBindTo,
	const FAsyncMessageId MessageId,
	FMessageCallbackFunc&& Callback,
	const FAsyncMessageBindingOptions& Options)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncMessageSystemBase::BindListener_Impl);
	
	// Do not allow the binding of new listeners during the shutdown of this message system
	if (bIsShuttingDown)
	{
		UE_LOGF(LogAsyncMessageSystem, Error, "[%s] Attempting to bind an message listener to message '%ls' during shutdown.",
			__func__, *MessageId.ToString());
		
		return false;
	}

	// Don't allow you to bind to an invalid message id
	if (!MessageId.IsValid())
	{
		UE_LOGF(LogAsyncMessageSystem, Error, "[%s] Attempting to bind an message listener to invalid message name '%ls'",
			__func__, *MessageId.ToString());
		
		return false;
	}

	if (!HandleToBindTo.IsValid())
	{
		UE_LOGF(LogAsyncMessageSystem, Error, "[%s] Attempting to bind message '%ls' to invalid handle '%ls'",
			__func__, *MessageId.ToString(), *HandleToBindTo.ToString());
		
		return false;
	}

	// Let the message store know that this message will be listened for by this binding option
	MessageStore.AddMessageToBinding(MessageId, Options);

	// Queue this listener for binding the next time that this system is processed.
	PendingBoundListenerQueue.Enqueue(
		FPendingBoundListener<FMessageCallbackFunc> 
		{
			.MessageId = MessageId, 
			.Data = FAsyncMessageBindingEndpoint::FAsyncMessageIndividualListener<FMessageCallbackFunc>
				{
					.Handle = HandleToBindTo,
					.Callback = MoveTempIfPossible(Callback)
				}, 
			.Options = Options
		});

	// Successfully queued for binding
	return true;
}

void FAsyncMessageSystemBase::UnbindListener_Impl(const FAsyncMessageHandle& HandleToUnbind)
{
	if (HandleToUnbind.IsValid())
	{
		MessageHandlesPendingRemoval.Enqueue(HandleToUnbind);	
	}
}

bool FAsyncMessageSystemBase::QueueMessageForBroadcast_Impl(
	const FAsyncMessageId MessageId,
	FConstStructView PayloadData,
	TWeakPtr<FAsyncMessageBindingEndpoint> BindingEndpoint)
{
	// Do not allow the queue of new listeners during the shutdown of this message system
	if (bIsShuttingDown)
	{
		UE_LOGF(LogAsyncMessageSystem, Error, "[%s] Attempting to queue an message '%ls' during shutdown. The message will not be queued",
			__func__, *MessageId.ToString());
		
		return false;
	}

	// Don't allow you to queue an invalid message id
	if (!MessageId.IsValid())
	{
		UE_LOGF(LogAsyncMessageSystem, Error, "[%s] Attempting to queue an invalid message id  '%ls', the message will not be queued",
			__func__, *MessageId.ToString());
		
		return false;
	}
	
	// Keep in mind that this function will likely be called from many different threads.
	// That is why we store our message message queue in a TMpscQueue, because there are multiple
	// producers (the systems queuing messages) and one consumer (this message system, processing them)

	// The message system will only store WEAK pointers to the bound listeners, and only VIEWS to the payload data.
	// This means that it is the responsibility of the system calling this "Queue" function to maintain ownership
	// of the payload data and ensure that it is in scope for any listeners that may want to access it.

	// Note: We have FStructView here, which should give us some reflected data about the type of
	// payload we are receiving.

	const double QueueTime = FApp::GetCurrentTime();
	const uint64 QueuedFrame = GFrameCounter;
	const uint32 ThreadQueuedFrom = FPlatformTLS::GetCurrentThreadId();
	const uint32 MessageSequence = NextMessageSequence++;
	
	uint32 NumQueuesAddedTo = 0u;
	TArray<FAsyncMessageBindingOptions> BoundTypes;
	
	// Push a new async message for this message and all it's parents.
	FAsyncMessage NewMessage
		(
			/* .MessageId= */ MessageId,
			/* .MessageSourceId= */ MessageId,
			/* .QueueTime= */ QueueTime,
			/* .QueueFrame= */ QueuedFrame,
			/* .ThreadQueuedFrom= */ ThreadQueuedFrom,
			/* .SequenceId= */ MessageSequence,		
			/* Payload= */ MoveTemp(PayloadData),		// <--- Note: The payload data is going to be copied for each message in the queue
			/* BindingEndpoint= */ BindingEndpoint
		);
			
	// Queue this message to the parent message
	NumQueuesAddedTo += MessageStore.EnqueueMessage(MoveTemp(NewMessage), OUT BoundTypes);
	
	UE_LOGF(LogAsyncMessageSystem, Verbose, "[%s] message '%ls' was added to %u binding queues", 
		__func__, *MessageId.ToString(), NumQueuesAddedTo);

	// Let subclasses know that an message was just queued and for what binding options. This allows them to 
	// do things like spin up async tasks (or UE::Tasks) in order to start processing that message queue
	PostQueueMessage(MessageId, BoundTypes);
	
	// Return true if there were any bound listeners to this message. 
	// False if the message was not added to any message queues
	return NumQueuesAddedTo > 0u;
}

void FAsyncMessageSystemBase::PreProcessMessagesQueue(const FAsyncMessageBindingOptions& Options)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncMessageSystemBase::PreProcessMessagesQueue);

	// Remove any message handles that have been unbound
	ProcessUnbindHandleRequests();

	// Process any deferred bindings which have been requested
	ProcessListenersPendingBinding();	
}

void FAsyncMessageSystemBase::ProcessMessageQueueForBinding_Impl(const FAsyncMessageBindingOptions& Options)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncMessageSystemBase::ProcessMessageQueueForBinding_Impl);

	if (bIsShuttingDown)
	{
		UE_LOGF(LogAsyncMessageSystem, Error, "[%s] Attempting to Process Message Queue during shutdown, exiting", __func__);
		return;
	}
	
	// Reset the message sequence ID which we are processing this frame
	NextMessageSequence = 0u;

	// Lock the message listener maps so that listeners don't get added/removed
	// in the middle of processing, which would cause a data race
	FScopeLock ListenerLock(&MessageListenerMapCS);
	
	uint32 ProcessedMessages = 0u;
	uint32 MessagesCalled = 0u;
	
	// Pop off of the message queue and process each message
	TOptional<FAsyncMessage> OptionalNextMessage = MessageStore.GetNextMessageForBindingOption(Options);
	while (OptionalNextMessage.IsSet())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncMessageSystemBase::ProcessMessageQueueForBinding_Impl::SingleMessage);

		FAsyncMessage& Message = OptionalNextMessage.GetValue();

		// TODO: If we define a "lifetime" for messages, we could grab the current frame/time
		// here and check against it to see if enough time has passed and we should just "throw
		// out" the message
		//
		//const double CurrentTime = FApp::GetCurrentTime();
		//const uint64 CurrentFrame = GFrameCounter;

		TSharedPtr<FAsyncMessageBindingEndpoint> MessageHandler = Message.GetBindingEndpoint();

		// Only process messages with valid handlers.
		// A message handler might be invalid the handler goes out of scope
		// (gets GC'd, or is otherwise destroyed after a message is queued)
		if (MessageHandler.IsValid())
		{
			// For each message in the hierarchy of this message...
			FAsyncMessageId::WalkMessageHierarchy(Message.GetMessageId(), [&](const FAsyncMessageId CurrentMessageId)
			{
				// Set this message ID to the current message id
				Message.SetMessageId(CurrentMessageId);
						
				// If we know of any bound listeners to this message, notify them.
				if (const FAsyncMessageBindingEndpoint::FAsyncMessageBoundData* BoundData = MessageHandler->GetBoundDataForMessage(Message.GetMessageId()))
				{
					// Look for any listeners associated with this binding option
					if (const TArray<FAsyncMessageBindingEndpoint::FAsyncMessageIndividualListener<FMessageCallbackFunc>>* Listeners = BoundData->ListenerMap.Find(Options))
					{
						for (const FAsyncMessageBindingEndpoint::FAsyncMessageIndividualListener<FMessageCallbackFunc>& Listener : *Listeners)
						{
							// As long as the callback for this listener is valid, broadcast it
							if (Listener.Callback)
							{
								// Broadcast to the listener!					
								Listener.Callback(Message);
						
								MessagesCalled++;	
							}
							// Otherwise, this callback is no long valid (the lambda may have gone out of scope)
							// and we should unbind it so that we don't bother trying to call it again
							else
							{
								UnbindListener(Listener.Handle);
							}
						}
					}
				}
			});	
		}
		
		OptionalNextMessage = MessageStore.GetNextMessageForBindingOption(Options);
	}

	UE_CLOGF(ProcessedMessages > 0, LogAsyncMessageSystem, VeryVerbose, "[%s] Processed %u messages. Called %u message listeners.", 
		__func__, ProcessedMessages, MessagesCalled);
}

void FAsyncMessageSystemBase::ProcessUnbindHandleRequests()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncMessageSystemBase::ProcessUnbindHandleRequests);

	if (MessageHandlesPendingRemoval.IsEmpty())
	{
		return;
	}

	FScopeLock MessageListenerMapLock(&MessageListenerMapCS);

	TOptional<FAsyncMessageHandle> HandleToRemoveOpt = MessageHandlesPendingRemoval.Dequeue();
	while (HandleToRemoveOpt.IsSet())
	{
		FAsyncMessageHandle& HandleToRemove = HandleToRemoveOpt.GetValue();
		
		if (TSharedPtr<FAsyncMessageBindingEndpoint> Endpoint = HandleToRemove.GetBindingEndpoint())
		{
			if (FAsyncMessageBindingEndpoint::FAsyncMessageBoundData* FoundBindingData = Endpoint->GetBoundDataForMessage(HandleToRemove.GetBoundMessageId()))
			{
				// Check the listener map for this message...
				for (auto& BindingPair : FoundBindingData->ListenerMap)
				{
					TArray<FAsyncMessageBindingEndpoint::FAsyncMessageIndividualListener<FMessageCallbackFunc>>& Listeners = BindingPair.Value;

					// Remove any listeners whose handles have been marked for unbinding
					for (int32 i = Listeners.Num() - 1; i >= 0; --i)
					{
						if (Listeners[i].Handle == HandleToRemove)
						{
							MessageStore.RemoveMessageFromBinding(Listeners[i].Handle.GetBoundMessageId(), BindingPair.Key);
							Listeners.RemoveAt(i);
						}
					}
				}
			}
			else
			{
				// If we get here, then the current handle has been unbound before we have actually had a chance to fully set up its binding yet.
				// For example, if you call Bind/Unbind immediately in a blueprint node. 
				// Keep track of these handles which we couldn't fully unbind yet so that we don't bother to set up their bindings.
				UE_LOGF(LogAsyncMessageSystem, Verbose, "Handle: %ls -- No binding data found. That means this listener has not been fully added yet. We will prevent it from being added instead of removing it.", *HandleToRemove.ToString());				
				IncompleteBoundHandlesToRemove.Add(HandleToRemove);
			}
		}
		else
		{
			UE_LOGF(LogAsyncMessageSystem, Error, "Attempting to unbind from an invalid endpoint. Failed to remove listeners for handle: %ls", *HandleToRemove.ToString());
		}

		// Get the next handle to remove
		HandleToRemoveOpt = MessageHandlesPendingRemoval.Dequeue();
	}
}

void FAsyncMessageSystemBase::ProcessListenersPendingBinding()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncMessageSystemBase::ProcessListenersPendingBinding);

	if (PendingBoundListenerQueue.IsEmpty())
	{
		return;
	}

	// We will lock the listener map and binding of messages to allow for multiple threads
	// to add listeners at the same time safely
	FScopeLock ListenerLock(&MessageListenerMapCS);

	// Iterate through the pending listeners and add them to our BoundMessageListenerMap Map.
	TOptional<FPendingBoundListener<FMessageCallbackFunc>> ListenerToBind = PendingBoundListenerQueue.Dequeue();
	while (ListenerToBind.IsSet())
	{
		TSharedPtr<FAsyncMessageBindingEndpoint> MessageHandler = ListenerToBind->Data.Handle.GetBindingEndpoint();

		// Only finish setting up the bindings for handles which have not been unbound already.
		if (!IncompleteBoundHandlesToRemove.Contains(ListenerToBind->Data.Handle))
		{
			// Bind a new listener to the message handler
			FAsyncMessageBindingEndpoint::FAsyncMessageBoundData& MessageData = MessageHandler->FindOrAddMessageData(ListenerToBind->MessageId);

			// Add a new listener which we can broadcast the callback to later
			MessageData.ListenerMap.FindOrAdd(ListenerToBind->Options).Add(MoveTemp(ListenerToBind->Data));
		}				
		else
		{
			UE_LOGF(LogAsyncMessageSystem, Verbose, "Skipping pending binding to handle %ls because it was already marked for unbinding.", *ListenerToBind->Data.Handle.ToString());
		}

		ListenerToBind = PendingBoundListenerQueue.Dequeue();
	}

	// We have processed all the pending bound listeners so we can now empty this
	IncompleteBoundHandlesToRemove.Empty();

	// After iterating and dequeue'ing off the listener map it should be empty
	check(PendingBoundListenerQueue.IsEmpty());
}

FAsyncMessageHandle FAsyncMessageSystemBase::GenerateNextValidMessageHandle(const FAsyncMessageId ForMessageId, TWeakPtr<FAsyncMessageBindingEndpoint> Endpoint)
{
	// Increment to the next handle, checking if its value is zero.
	// If the value is zero, that means that we have hit uint32 max and overflowed back around
	// and we should increment the handle again to get a valid index
	uint32 HandleValue = ++NextMessageHandleId;
	if (HandleValue == 0)
	{
		HandleValue = ++NextMessageHandleId;
		
		// It is likely non-fatal if the number of handles has grown this pointer because the older handles will
		// be safe to be re-used, but log a warning here in case
		UE_LOGF(LogAsyncMessageSystem, Warning, "The Async Message Handle index has been wrapped!");
	}
	
	return FAsyncMessageHandle(HandleValue, ForMessageId, Endpoint.IsValid() ? Endpoint : DefaultBindingEndpoint);
}

// Dev test function only to test creation of valid handles, just to make unit testing easier.
#if WITH_DEV_AUTOMATION_TESTS
FAsyncMessageHandle FAsyncMessageSystemBase::GenerateHandleAtIndex(const uint32 Index, const FAsyncMessageId& ForId, TWeakPtr<FAsyncMessageBindingEndpoint> Endpoint)
{
	return FAsyncMessageHandle(Index, ForId, Endpoint);
}
#endif