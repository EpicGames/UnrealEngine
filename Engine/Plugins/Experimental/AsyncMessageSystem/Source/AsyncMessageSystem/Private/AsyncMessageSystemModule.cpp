// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncMessageSystemModule.h"
#include "Modules/ModuleManager.h"

bool FAsyncMessageSystemModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded("AsyncMessageSystem");
}

IMPLEMENT_MODULE(FAsyncMessageSystemModule, AsyncMessageSystem)
