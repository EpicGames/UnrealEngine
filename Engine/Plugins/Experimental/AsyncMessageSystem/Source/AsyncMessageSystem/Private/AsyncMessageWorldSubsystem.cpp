// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncMessageWorldSubsystem.h"
#include "AsyncGameplayMessageSystem.h"
#include "AsyncMessageSystemLogs.h"
#include "AsyncMessageDeveloperSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncMessageWorldSubsystem)

void UAsyncMessageWorldSubsystem::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UAsyncMessageWorldSubsystem* ThisPtr = CastChecked<UAsyncMessageWorldSubsystem>(InThis);
	
	if (ThisPtr->MessageSystem.IsValid())
	{
		ThisPtr->MessageSystem->AddReferencedObjects(InThis, Collector);
	}
}

namespace UE::Private
{
	static bool ShouldCreateWorldSubsystem(const UWorld* ForWorld)
	{
		if (!ForWorld)
		{
			return false;
		}
		const UAsyncMessageDeveloperSettings* Settings = GetDefault<UAsyncMessageDeveloperSettings>();
	
		// Let you disable for editor worlds
		if (ForWorld->WorldType == EWorldType::Editor)
		{
			return Settings->GetShouldEnableWorldSubsystemInEditor();
		}
	
		return Settings->GetShouldEnableWorldSubsystem();
	}
};

bool UAsyncMessageWorldSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	if (!UE::Private::ShouldCreateWorldSubsystem(Cast<UWorld>(Outer)))
	{
		return false;
	}

	// Fall back to default behavior for editor, game, and pie worlds
	return Super::ShouldCreateSubsystem(Outer);
}

void UAsyncMessageWorldSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	InitMessageSystem();
}

void UAsyncMessageWorldSubsystem::Deinitialize()
{
	Super::Deinitialize();

	ShutdownMessageSystem();
}

void UAsyncMessageWorldSubsystem::InitMessageSystem()
{
	check(!MessageSystem.IsValid());
	
	UE_LOGF(LogAsyncMessageSystem, Verbose, "[%s] Init world message system for world '%ls'", __func__, *GetNameSafe(GetWorld()));

	MessageSystem = FAsyncMessageSystemBase::CreateMessageSystem<FAsyncGameplayMessageSystem>(GetWorld());
}

void UAsyncMessageWorldSubsystem::ShutdownMessageSystem()
{
	if (ensure(MessageSystem.IsValid()))
	{
		UE_LOGF(LogAsyncMessageSystem, Verbose, "[%s] Shutdown world message system for world '%ls'", __func__, *GetNameSafe(GetWorld()));
		
		MessageSystem->Shutdown();
		
		MessageSystem.Reset();		
	}
	
	OnShutdownMessageSystem.Broadcast();
}
