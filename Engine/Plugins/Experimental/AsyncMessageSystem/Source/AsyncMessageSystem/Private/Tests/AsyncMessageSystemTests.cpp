// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncMessageSystemTests.h"

#include "AsyncMessageHandle.h"
#include "AsyncMessageSystemBase.h"
#include "AsyncMessageBindingEndpoint.h"
#include "Misc/AutomationTest.h"
#include "NativeGameplayTags.h"
#include "Tasks/Task.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncMessageSystemTests)

UE_DEFINE_GAMEPLAY_TAG_COMMENT(InternalTestTag_Invalid, "AsyncMessages.Internal.test.invalid", "A test gameplay tag utilized in the async message system unit tests")
UE_DEFINE_GAMEPLAY_TAG_COMMENT(InternalTestTag_A, "AsyncMessages.Internal.test.a", "A test gameplay tag utilized in the async message system unit tests")
UE_DEFINE_GAMEPLAY_TAG_COMMENT(InternalTestTag_B, "AsyncMessages.Internal.test.b", "A test gameplay tag utilized in the async message system unit tests")

UE_DEFINE_GAMEPLAY_TAG_COMMENT(InternalTestTag_Child, "AsyncMessages.Internal.test.child", "A test gameplay tag utilized in the async message system unit tests")

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::AsyncMessageSystem
{
	constexpr EAutomationTestFlags QuickTestFlags = EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter;
	constexpr EAutomationTestFlags StressTestFlags = EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::StressFilter;
	
	// Test message system interface impl
	class FTestMessageSystem : public FAsyncMessageSystemBase
	{
	protected:
		virtual void Startup_Impl() override
		{
			bHasStarted = true;
		}

		virtual void Shutdown_Impl() override
		{
			bHasShutdown = true;
		}

		virtual void PostQueueMessage(const FAsyncMessageId MessageId, const TArray<FAsyncMessageBindingOptions>& OptionsBoundTo) override
		{
			// Dont do anything by default here in this simple test message system
		}
		
		bool bHasStarted = false;
		bool bHasShutdown = false;
	public:
		[[nodiscard]] FAsyncMessageHandle GenerateHandleAtIndex(
		   const uint32 Index,
		   const FAsyncMessageId& ForId)
		{
			return FAsyncMessageSystemBase::GenerateHandleAtIndex(Index, ForId, DefaultBindingEndpoint);
		}
		
		const bool HasStarted() const { return bHasStarted; }
		const bool HasShutdown() const { return bHasShutdown && bIsShuttingDown; }
		
		/**
		 * Returns true if the handle currently has any listeners bound to it in the message map
		 */
		bool IsHandleBound(const FAsyncMessageHandle& Handle) const
		{
			FScopeLock ListenerLock(&MessageListenerMapCS);
			
			return DefaultBindingEndpoint->IsHandleBound(Handle);	
		}

		uint32 GetNumberOfListeners() const
		{
			FScopeLock ListenerLock(&MessageListenerMapCS);

			return DefaultBindingEndpoint->GetNumberOfBoundListeners();
		}
		
		void RunOnce(const TArray<FAsyncMessageBindingOptions>& OptsToProcess = { FAsyncMessageBindingOptions{} })
		{
			for (const FAsyncMessageBindingOptions& Options : OptsToProcess)
			{
				ProcessMessageQueueForBinding(Options);
			}			
		}

		void Test_Shutdown()
		{
			Shutdown();
		}

		FAsyncMessageHandle GenerateTestHandle()
		{
			return GenerateNextValidMessageHandle({InternalTestTag_Invalid}, DefaultBindingEndpoint);
		}
	};
	

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(TMessageHandleDefaults, "AsyncMessagePassing.MessageHandles", QuickTestFlags)
	bool TMessageHandleDefaults::RunTest(const FString& InParameters)
	{
		TSharedPtr<FTestMessageSystem, ESPMode::ThreadSafe> TestSystem = FAsyncMessageSystemBase::CreateMessageSystem<FTestMessageSystem>();
		// Test defaults
		{
			FAsyncMessageHandle DefaultHandle = {};
			TestFalse(TEXT("Default async message handle is invalid"), DefaultHandle.IsValid());

			const FString DefaultHandleToString = DefaultHandle.ToString();
			const FString ExpectedToString = TEXT("0");
			TestTrue(TEXT("Default handle has correct ToString"), DefaultHandleToString == ExpectedToString);
		}

		// Test Invalidate
		{
			FAsyncMessageHandle TestHandle = TestSystem->GenerateHandleAtIndex(123, {InternalTestTag_Invalid});
			TestTrue(TEXT("Test handle starts as a valid handle"), TestHandle.IsValid());
		}

		// Test GetId()
		{
			constexpr uint32 TestHandleIdx = 456;
			FAsyncMessageHandle Handle = TestSystem->GenerateHandleAtIndex(TestHandleIdx, {InternalTestTag_Invalid});
			TestTrue(TEXT("Handle Is Valid"), Handle.IsValid());
			
			TestTrue(TEXT("Handle Is the same as the given index"), TestHandleIdx == Handle.GetId());
		}

		// Test ToString()
		{
			constexpr uint32 TestHandleIdx = 456;
			FAsyncMessageHandle Handle = TestSystem->GenerateHandleAtIndex(TestHandleIdx, {InternalTestTag_Invalid});
			
			const FString DefaultHandleToString = Handle.ToString();
			const FString ExpectedToString = TEXT("456");
			TestTrue(TEXT("Handle handle has correct ToString"), DefaultHandleToString == ExpectedToString);
		}

		// Test handle == operator
		{
			FAsyncMessageHandle A = TestSystem->GenerateHandleAtIndex(789, {InternalTestTag_Invalid});
			FAsyncMessageHandle B = TestSystem->GenerateHandleAtIndex(789, {InternalTestTag_Invalid});
			TestTrue(TEXT("Async Message Handle == operator works"), A == B);
		}

		// Test handle != operator
		{
			FAsyncMessageHandle A = TestSystem->GenerateHandleAtIndex(123, {InternalTestTag_Invalid});
			FAsyncMessageHandle B = TestSystem->GenerateHandleAtIndex(789, {InternalTestTag_Invalid});
			TestTrue(TEXT("Async Message Handle != operator works"), A != B);
		}

		TestSystem->Shutdown();
		TestTrue(TEXT("MessageSystem shutdown has been called"), TestSystem->HasShutdown());

		TestSystem.Reset();
		
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(TMessageParentHandles, "AsyncMessagePassing.MessageParentHandle", QuickTestFlags)
	bool TMessageParentHandles::RunTest(const FString& InParameters)
	{
		static const FName ExpectedChildMessageName					= "AsyncMessages.Internal.test.child";
		static const FName ExpectedParentMessageName				= "AsyncMessages.Internal.test";
		static const FName ExpectedGrandparentMessageName			= "AsyncMessages.Internal";
		static const FName ExpectedGreatGrandparentMessageName		= "AsyncMessages";
		static const FName ExpectedGreatGreatGrandparentMessageName	= NAME_None;
		
		// Start with a child
		static const FAsyncMessageId GrandChildMessage { InternalTestTag_Child };
		TestTrue(TEXT("Child name is correct"), ExpectedChildMessageName == GrandChildMessage.GetMessageName());
		
		// Ensure the parent is correct
		const FAsyncMessageId TestParentId = GrandChildMessage.GetParentMessageId();
		TestTrue(TEXT("Test Parent ID is correct"), ExpectedParentMessageName == TestParentId.GetMessageName());

		// Ensure the grandparent is correct
		const FAsyncMessageId TestGrandparentId = TestParentId.GetParentMessageId();
		TestTrue(TEXT("Test Grand Parent ID is correct"), ExpectedGrandparentMessageName == TestGrandparentId.GetMessageName());

		// Great grand parent, which should be the root tag
		const FAsyncMessageId TestGreatGrandparentId = TestGrandparentId.GetParentMessageId();
		TestTrue(TEXT("Test Great Grand Parent ID is correct"), ExpectedGreatGrandparentMessageName == TestGreatGrandparentId.GetMessageName());

		// Great great grandparent should be empty
		const FAsyncMessageId TestGreatGreatGrandparentId = TestGreatGrandparentId.GetParentMessageId();
		TestTrue(TEXT("Test Great Grand Parent ID is correct"), ExpectedGreatGreatGrandparentMessageName == TestGreatGreatGrandparentId.GetMessageName());
		
		return true;
	}

	const FAsyncMessageId TestMessageId_A = {InternalTestTag_A};
	
	struct FTestListener
	{
	public:
		void Callback_Message_A(const FAsyncMessage& Message)
		{
			FConstStructView DataView = Message.GetPayloadView();
			const FTest_Payload_A* Data = Message.GetPayloadData<const FTest_Payload_A>();
			if (Data)
			{
				CallbackMutation += Data->IncrementAmount;
			}
		}

		int32 CallbackMutation = 0;
	};

	// Test that the shutdown flag is correctly being set when we call Shutdown
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(TMessageSystemShutdownTest, "AsyncMessagePassing.StartupShutdown", QuickTestFlags)
	bool TMessageSystemShutdownTest::RunTest(const FString& InParameters)
	{
		// Startup a new message system...
		TSharedPtr<FTestMessageSystem, ESPMode::ThreadSafe> TestSystem = FAsyncMessageSystemBase::CreateMessageSystem<FTestMessageSystem>();
		TestTrue(TEXT("MessageSystem startup has been called"), TestSystem->HasStarted());

		TestSystem->RunOnce();
		
		TestSystem->Shutdown();
		TestTrue(TEXT("MessageSystem shutdown has been called"), TestSystem->HasShutdown());

		TestSystem.Reset();
		return true;
	}
	
	// Test that we can bind an message onto a non-UObject shared pointer and that it receives the message after we pump the message queue
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(TMessageSystemBindingSharedPtrTest, "AsyncMessagePassing.Binding.SharedPtr", QuickTestFlags)
	bool TMessageSystemBindingSharedPtrTest::RunTest(const FString& InParameters)
	{
		// Startup a new message system...
		TSharedPtr<FTestMessageSystem, ESPMode::ThreadSafe> TestSystem = FAsyncMessageSystemBase::CreateMessageSystem<FTestMessageSystem>();
		TestTrue(TEXT("MessageSystem startup has been called"), TestSystem->HasStarted());

		// Set up a test listener object with some dummy data
		TSharedPtr<FTestListener> TestListener = MakeShared<FTestListener>();
		TestTrue(TEXT("Callback Object mutated successfully"), TestListener->CallbackMutation == 0);

		const int32 TestStartValue = TestListener->CallbackMutation;
		
		// Bind an message callback to the test listener
		FAsyncMessageHandle ListenerHandle = TestSystem->BindListener(TestMessageId_A, TestListener.ToWeakPtr(), &FTestListener::Callback_Message_A);
		
		// This handle will be in the pending queue
		TestTrue(TEXT("A valid listener handle was provided to binding"), ListenerHandle.IsValid());

		// Make some instance struct payload
		const int32 TestAmountToAdd = 1563875499;
		
		FTest_Payload_A PayloadData = {};
		PayloadData.IncrementAmount = TestAmountToAdd;
		FInstancedStruct PayloadDataInstance = FInstancedStruct::Make<FTest_Payload_A>(PayloadData);
		
		// Queue a new message for broadcast!
		TestSystem->QueueMessageForBroadcast(TestMessageId_A, PayloadDataInstance);

		// Next, simulate the message system being ticked once
		// This should process our delegate bindings and execute anything in the message queue
		TestSystem->RunOnce();

		// Ensure that the callback incremented the data with the correct number
		TestTrue(TEXT("Callback Object mutated successfully"), TestListener->CallbackMutation == (TestStartValue + TestAmountToAdd));

		// Remove the bound listener and test that is was successfully
		TestSystem->UnbindListener(ListenerHandle);

		TestSystem->RunOnce();
		TestFalse(TEXT("Unregister the listener handle"), TestSystem->IsHandleBound(ListenerHandle));

		// All done
		TestSystem->Shutdown();
		TestSystem.Reset();
		return true;
	}
	
	// Test to make sure that you can bind an message listener whilst the message system is currently processing
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(TMessageSystemNestedBinding, "AsyncMessagePassing.Binding.NestedMessage", QuickTestFlags)
	bool TMessageSystemNestedBinding::RunTest(const FString& InParameters)
	{
		static const FAsyncMessageId NestedMessageId = {InternalTestTag_B};

		struct FTestNestedObject
		{
			void TestNestedBinding(const FAsyncMessage& Message)
			{
				// We should be able to add a test binding in response to 
				if (const FNested_Payload* Data = Message.GetPayloadData<const FNested_Payload>())
				{
					if (TSharedPtr<FAsyncMessageSystemBase> Sys = Data->MessageSystem.Pin())
					{
						// Bind a nested listener here
						const FAsyncMessageHandle Handle = Sys->BindListener(TestMessageId_A, [](const FAsyncMessage& Message)
						{
							// we dont need to do anything here, just ensure that the actual BindListener function works
						});	
					}
				}
			}
		};
		
		// Startup a new message system...
		TSharedPtr<FTestMessageSystem, ESPMode::ThreadSafe> TestSystem = FAsyncMessageSystemBase::CreateMessageSystem<FTestMessageSystem>();
		TestTrue(TEXT("MessageSystem startup has been called"), TestSystem->HasStarted());
		
		TSharedPtr<FTestNestedObject> TestObject = MakeShared<FTestNestedObject>();

		const FAsyncMessageHandle Handle = TestSystem->BindListener(NestedMessageId, TestObject.ToWeakPtr(), &FTestNestedObject::TestNestedBinding);
		
		TestTrue(TEXT("System has a no listeners yet, they are in the pending queue"), TestSystem->GetNumberOfListeners() == 0);

		FNested_Payload Payload = { .MessageSystem = TestSystem };
		FInstancedStruct PayloadInstance = FInstancedStruct::Make<FNested_Payload>(Payload);

		TestSystem->QueueMessageForBroadcast(NestedMessageId, PayloadInstance);

		// Running once should put the nested binding in the "pending" bindings queue
		TestSystem->RunOnce();

		TestTrue(TEXT("System has a single listener"), TestSystem->GetNumberOfListeners() == 1);

		// Running again should process that pending bindings queue and add the new listener
		TestSystem->RunOnce();
		
		TestTrue(TEXT("System has two listeneres now, one is nested"), TestSystem->GetNumberOfListeners() == 2);

		TestSystem->Shutdown();
		TestSystem.Reset();
		return true;
	}

	// Test that we can bind an message onto a UObject and that it receives the message after we pump the message queue
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(TMessageSystemBindingUObjectTest, "AsyncMessagePassing.Binding.UObject", QuickTestFlags)
	bool TMessageSystemBindingUObjectTest::RunTest(const FString& InParameters)
	{
		TSharedPtr<FTestMessageSystem, ESPMode::ThreadSafe> TestSystem = FAsyncMessageSystemBase::CreateMessageSystem<FTestMessageSystem>();
		TestTrue(TEXT("MessageSystem startup has been called"), TestSystem->HasStarted());

		TObjectPtr<UTestAsyncObject> TestObject = NewObject<UTestAsyncObject>();
		TestTrue(TEXT("Created test object"), IsValid(TestObject));
		const int32 TestStartValue = TestObject->TestValue;

		// Bind a listener to the test object
		FAsyncMessageHandle ListenerHandle = TestSystem->BindListener(TestMessageId_A, TWeakObjectPtr<UTestAsyncObject>(TestObject), &UTestAsyncObject::CallbackFunction);
		TestTrue(TEXT("A valid listener handle was provided to binding"), ListenerHandle.IsValid());
		
		// Create a test payload instanced struct
		const int32 TestAmountToAdd = 7;
		
		FTest_Payload_A PayloadData = {};
		PayloadData.IncrementAmount = TestAmountToAdd;
		FInstancedStruct PayloadDataInstance = FInstancedStruct::Make<FTest_Payload_A>(PayloadData);

		// Queue a new message for broadcast!
		TestSystem->QueueMessageForBroadcast(TestMessageId_A, PayloadDataInstance);

		// Next, simulate the message system being ticked once
		// This should process our delegate bindings and execute anything in the message queue
		TestSystem->RunOnce();

		TestTrue(TEXT("Callback UObject mutated successfully"), TestObject->TestValue == (TestStartValue + TestAmountToAdd));

		TestSystem->Shutdown();
		TestSystem.Reset();
		
		return true;
	}	

	// Test to make sure that a single message can be queued and broadcast to multiple different kinds of binding options.
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(TMessageSystemBindingMultipleOptions, "AsyncMessagePassing.Binding.MultipleOptions", QuickTestFlags)
	bool TMessageSystemBindingMultipleOptions::RunTest(const FString& InParameters)
	{
		TSharedPtr<FTestMessageSystem, ESPMode::ThreadSafe> TestSystem = FAsyncMessageSystemBase::CreateMessageSystem<FTestMessageSystem>();
		TestTrue(TEXT("MessageSystem startup has been called"), TestSystem->HasStarted());

		auto TestLambda = [&](const FAsyncMessage& Message)
		{
			const FTest_Payload_A* Data = Message.GetPayloadData<const FTest_Payload_A>();
			TestNotNull(TEXT("Const Payload data is valid "), Data);

			FConstStructView DataView = Message.GetPayloadView();
			TestTrue(TEXT("Const Payload view is valid"), DataView.IsValid());			
		};

		// Bind some test listeners which will ensure that the payload data is valid
		const TArray<FAsyncMessageBindingOptions> BindingsToUse = 
		{
			FAsyncMessageBindingOptions(ETickingGroup::TG_PrePhysics),
			FAsyncMessageBindingOptions(ETickingGroup::TG_DuringPhysics),
			FAsyncMessageBindingOptions(ETickingGroup::TG_PostPhysics),
			FAsyncMessageBindingOptions(ETickingGroup::TG_PostUpdateWork),
			FAsyncMessageBindingOptions(ENamedThreads::HighTaskPriority),
			FAsyncMessageBindingOptions(ENamedThreads::GameThread),
			FAsyncMessageBindingOptions(ENamedThreads::RHIThread),
			FAsyncMessageBindingOptions(UE::Tasks::ETaskPriority::Default, UE::Tasks::EExtendedTaskPriority::Inline),
			FAsyncMessageBindingOptions(UE::Tasks::ETaskPriority::ForegroundCount, UE::Tasks::EExtendedTaskPriority::TaskEvent),
			FAsyncMessageBindingOptions(UE::Tasks::ETaskPriority::BackgroundNormal, UE::Tasks::EExtendedTaskPriority::GameThreadHiPri),
		};
		
		for (const FAsyncMessageBindingOptions& Opts : BindingsToUse)
		{
			// Bind listeners to multiple different types
			TestSystem->BindListener(TestMessageId_A, TestLambda, Opts);
		}
		
		// Queue a test message for broadcasting
		FInstancedStruct PayloadDataInstance = FInstancedStruct::Make<FTest_Payload_A>(FTest_Payload_A{});
		TestSystem->QueueMessageForBroadcast(TestMessageId_A, PayloadDataInstance);

		// Actually run the system, which should process all the messages in the queue.
		TestSystem->RunOnce(BindingsToUse);

		TestSystem->Shutdown();
		TestSystem.Reset();
		return true;
	}

	// Test binding a simple C++ lambda 
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(TMessageSystemBindingLambdaTest, "AsyncMessagePassing.Binding.Lambda", QuickTestFlags)
	bool TMessageSystemBindingLambdaTest::RunTest(const FString& InParameters)
	{
		TSharedPtr<FTestMessageSystem, ESPMode::ThreadSafe> TestSystem = FAsyncMessageSystemBase::CreateMessageSystem<FTestMessageSystem>();
		TestTrue(TEXT("MessageSystem startup has been called"), TestSystem->HasStarted());

		// Create a test payload instanced struct
		const int32 TestAmountToAdd = 7;
		
		FTest_Payload_A PayloadData = {};
		PayloadData.IncrementAmount = TestAmountToAdd;
		FInstancedStruct PayloadDataInstance = FInstancedStruct::Make<FTest_Payload_A>(PayloadData);

		// This Test Data will be what we use to verify that the lambda function has run and successfully
		// read/write some data
		int32 TestDataToMutate = 5;
		const int32 TestStartValue = TestDataToMutate;
		
		// Bind a listener to the test lambda
		const FAsyncMessageHandle ListenerHandle = TestSystem->BindListener(TestMessageId_A, [&TestDataToMutate](const FAsyncMessage& Message)
		{
			if (const FTest_Payload_A* Data = Message.GetPayloadData<const FTest_Payload_A>())
			{
				TestDataToMutate += Data->IncrementAmount;
			}
		});

		// Queue a new message for broadcast!
		TestSystem->QueueMessageForBroadcast(TestMessageId_A, PayloadDataInstance);

		// Next, simulate the message system being ticked once
		// This should process our delegate bindings and execute anything in the message queue
		TestSystem->RunOnce();

		TestTrue(TEXT("Callback lambda mutated successfully"), TestDataToMutate == (TestStartValue + TestAmountToAdd));
		TestSystem->Shutdown();
		TestSystem.Reset();
		return true;
	}

	// Test to queue messages from multiple threads on a single message system
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(TMessageSystemBindingEndpointsTest, "AsyncMessagePassing.Binding.CustomEndpoints", QuickTestFlags)
	bool TMessageSystemBindingEndpointsTest::RunTest(const FString& InParameters)
	{
		TSharedPtr<FTestMessageSystem, ESPMode::ThreadSafe> TestSystem = FAsyncMessageSystemBase::CreateMessageSystem<FTestMessageSystem>();
		TestTrue(TEXT("MessageSystem startup has been called"), TestSystem->HasStarted());

		static const FAsyncMessageId TestMessageId { InternalTestTag_A };

		FAsyncMessageBindingOptions BindOptions = {};
		TArray<FAsyncMessageBindingOptions> BindingsToTick = { BindOptions };
		
		// Bind to the normal (default) endpoint
		uint32 DefaultEndpointValue = 0;
		{
			TestSystem->BindListener(TestMessageId, [&DefaultEndpointValue](const FAsyncMessage& Message)
			{
				DefaultEndpointValue++;
			},
			BindOptions);
		}

		// Create a custom binding endpoint
		TSharedPtr<FAsyncMessageBindingEndpoint> CustomEndpoint = MakeShared<FAsyncMessageBindingEndpoint>();
		uint32 CustomEndpointValue = 0;
		{
			TestSystem->BindListener(TestMessageId, [&CustomEndpointValue](const FAsyncMessage& Message)
			{
				CustomEndpointValue++;
			},
			BindOptions,
			CustomEndpoint);
		}

		// Queue a message for the default endpoint
		TestSystem->QueueMessageForBroadcast(TestMessageId, /*payload*/nullptr);

		// and tick the bindings
		TestSystem->RunOnce(BindingsToTick);

		TestEqual(TEXT("Default endpoint got called correctly"), DefaultEndpointValue, 1);
		TestEqual(TEXT("CustomEndpoint did not get called, as expected"), CustomEndpointValue, 0);

		// Queue a message for the custom endpoint...
		TestSystem->QueueMessageForBroadcast(TestMessageId, /*payload*/nullptr, CustomEndpoint);
		TestSystem->RunOnce(BindingsToTick);
		
		TestEqual(TEXT("Default endpoint did not get called, as expected"), DefaultEndpointValue, 1);
		TestEqual(TEXT("CustomEndpoint was called correctly"), CustomEndpointValue, 1);
		
		TestSystem->Shutdown();
		TestSystem.Reset();
		
		return true;
	}
	
	// Test that the handle of an message is successfully unbound
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(TMessageSystemUnBindingSingle, "AsyncMessagePassing.UnBinding.Single", QuickTestFlags)
	bool TMessageSystemUnBindingSingle::RunTest(const FString& InParameters)
	{
		// Startup a new message system...
		TSharedPtr<FTestMessageSystem, ESPMode::ThreadSafe> TestSystem = FAsyncMessageSystemBase::CreateMessageSystem<FTestMessageSystem>();
		TestTrue(TEXT("MessageSystem startup has been called"), TestSystem->HasStarted());
		
		const FAsyncMessageHandle ListenerHandle = TestSystem->BindListener(TestMessageId_A, [this](const FAsyncMessage& Message)
		{
			const uint64 Frame = Message.GetQueueFrame();
			TestTrue(TEXT("Testing inside the lambda"), Frame > 0);
		});

		// Similate running once, and ensure that the handle is still bound correctly
		TestSystem->RunOnce();
		TestTrue(TEXT("A valid listener handle was provided to binding"), ListenerHandle.IsValid());
		TestTrue(TEXT("Listener Handle Is bound"), TestSystem->IsHandleBound(ListenerHandle));

		// Remove the bound listener and test that is was successfully
		TestSystem->UnbindListener(ListenerHandle);
		TestSystem->RunOnce();
		TestFalse(TEXT("Unregister the listener handle"), TestSystem->IsHandleBound(ListenerHandle));

		TestSystem->RunOnce();
		
		TestFalse(TEXT("Listener handle is no longer valid  after running the system"), TestSystem->IsHandleBound(ListenerHandle));

		// All done
		TestSystem->Shutdown();
		TestSystem.Reset();
		return true;
	}

	// This test ensures that if you: 
	// 1. Bind a listener
	// 2. Call "Unbind" on it immediately, before the subsystem can tick
	// 3. An event is queued and fired for the same message you were bound it
	// 4. The listener from 1 does NOT get fired.
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(TMessageSystemUnBindingBeforeBindingFinishes, "AsyncMessagePassing.UnBinding.BeforeBindingFinishes", QuickTestFlags)
	bool TMessageSystemUnBindingBeforeBindingFinishes::RunTest(const FString& InParameters)
	{
		// Startup a new message system...
		TSharedPtr<FTestMessageSystem, ESPMode::ThreadSafe> TestSystem = FAsyncMessageSystemBase::CreateMessageSystem<FTestMessageSystem>();
		TestTrue(TEXT("MessageSystem startup has been called"), TestSystem->HasStarted());

		// Set up a listener which would modify some test data if called.
		int32 TestData = 0;
		const FAsyncMessageHandle ListenerHandle = TestSystem->BindListener(TestMessageId_A, [&TestData, this](const FAsyncMessage& Message)
		{
			TestData++;
			TestTrue(TEXT("Test failed. This delegate should never get called."), false);
		});

		// Note: When you the call Bind/Unbind listener functions, the handles are put a queue, so they will not immediately be made valid or invalid.

		// Ensure that the pre-tick state is correct. The handle shouldn't be bound yet because we have not ticked the subsystem
		TestFalse(TEXT("The handle is in the expected pre-tick state of being unbound"), TestSystem->IsHandleBound(ListenerHandle));
		
		// Unbind the listener now, before we tick and ProcessListenersPendingBinding gets called.
		TestSystem->UnbindListener(ListenerHandle);

		// Tick the subsytem, which should process the pending queue of listeners and bind the delegates internally.
		TestSystem->RunOnce();

		// Since we called Unbind immediately, the handle should never be fully bound.
		TestFalse(TEXT("The handle was correctly not bound during tick."), TestSystem->IsHandleBound(ListenerHandle));

		// Fire a message for the event which was bound/unbound. The test data should not be mutated
		TestSystem->QueueMessageForBroadcast(TestMessageId_A);
		TestSystem->RunOnce();

		// Ensure that our test data has not been modified after firing a delegate
		TestTrue(TEXT("The handle is in the expected post-tick state and is unbound"), TestData == 0);

		// All done
		TestSystem->Shutdown();
		TestSystem.Reset();
		return true;
	}
	
	// Test that the handle of an message is successfully unbound
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(TMessageSystemUnBindingMultiple, "AsyncMessagePassing.UnBinding.Multiple", QuickTestFlags)
	bool TMessageSystemUnBindingMultiple::RunTest(const FString& InParameters)
	{
		// Startup a new message system...
		TSharedPtr<FTestMessageSystem, ESPMode::ThreadSafe> TestSystem = FAsyncMessageSystemBase::CreateMessageSystem<FTestMessageSystem>();
		TestTrue(TEXT("MessageSystem startup has been called"), TestSystem->HasStarted());

		TArray<FAsyncMessageHandle> Handles;

		constexpr int32 NumHandlesToBind = 50;
		int32 TestNum = 0;
		for (int32 i = 0; i < NumHandlesToBind; i++)
		{
			const FAsyncMessageHandle ListenerHandle = TestSystem->BindListener(TestMessageId_A, [&TestNum](const FAsyncMessage& Message)
			{
				TestNum++;
			});
			Handles.Emplace(ListenerHandle);

			TestTrue(TEXT("Listener Handle Is bound"), ListenerHandle.IsValid());
		}

		TestSystem->QueueMessageForBroadcast(TestMessageId_A);
		
		// Simulate running once, and ensure that the handle is still bound correctly
		TestSystem->RunOnce();
		
		TestTrue(TEXT("Modifyed value correctly"), TestNum == NumHandlesToBind);
		
		for (FAsyncMessageHandle& ListenerHandle : Handles)
		{
			TestSystem->UnbindListener(ListenerHandle);
		}
		
		TestSystem->RunOnce();

		for (FAsyncMessageHandle& ListenerHandle : Handles)
		{
			TestFalse(TEXT("Unregister the listener handle"), TestSystem->IsHandleBound(ListenerHandle));
		}

		// All done
		TestSystem->Shutdown();
		TestSystem.Reset();
		return true;
	}

	////////////////////////////////////////////////////
	// Multi-threaded tests. These will test some core functionality and thread safety by spawning a
	// bunch of UE::Tasks on different worker threads, and then modifying/accessing the message system
	// through them.
	
	// Test to queue messages from multiple threads on a single message system
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(TMessageSystemMultiThreadedQueueMessagesTest, "AsyncMessagePassing.MultiThreaded.QueueMessages", StressTestFlags)
	bool TMessageSystemMultiThreadedQueueMessagesTest::RunTest(const FString& InParameters)
	{
		TSharedPtr<FTestMessageSystem, ESPMode::ThreadSafe> TestSystem = FAsyncMessageSystemBase::CreateMessageSystem<FTestMessageSystem>();
		TestTrue(TEXT("MessageSystem startup has been called"), TestSystem->HasStarted());

		FAsyncMessageHandle ListenerHandle = TestSystem->BindListener(TestMessageId_A, [](const FAsyncMessage& Message)
		{
			FConstStructView DataView = Message.GetPayloadView();
			const FTest_Payload_A* Data = Message.GetPayloadData<const FTest_Payload_A>();
			if (Data)
			{
				UE_LOGF(LogTemp, Log, "DataVal : %d", Data->IncrementAmount);
			}
			else
			{
				UE_LOGF(LogTemp, Error, "Failed to find the payload data, it has expired");
			}
		});
		
		struct FQueueMessageFunctor
		{
			TWeakPtr<FTestMessageSystem> MessageSys;

			int32 AmountToAdd = 0;
			
			void operator()()
			{
				TSharedPtr<FTestMessageSystem> StrongMessageSys = MessageSys.Pin();
				check(StrongMessageSys.IsValid());

				// It is expected that this instanced struct payload data here would go out of scope.
				FTest_Payload_A PayloadData = {};
				PayloadData.IncrementAmount = AmountToAdd;
				FInstancedStruct PayloadDataInstance = FInstancedStruct::Make<FTest_Payload_A>(PayloadData);
				
				StrongMessageSys->QueueMessageForBroadcast(TestMessageId_A, PayloadDataInstance);
			}
		};

		TArray<UE::Tasks::FTask> PendingTasks;
		constexpr int32 NumTasksToSpawn = 5000;
		for (int32 i = 0; i < NumTasksToSpawn; ++i)
		{
			UE::Tasks::FTask T = UE::Tasks::Launch(UE_SOURCE_LOCATION, FQueueMessageFunctor { .MessageSys = TestSystem, .AmountToAdd = i });
			PendingTasks.Emplace(T);
		}

		// Wait for all the tasks which queue messages to complete
		UE::Tasks::Wait(PendingTasks);

		TestSystem->RunOnce();

		TestSystem->Shutdown();
		TestSystem.Reset();
		
		return true;
	}

	// A test which will spin up several tasks on different threads to attempt to bind listeners to messages
	// on each thread. This will test that we can have different listeners binding on different threads
	// for the same message system
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(TMessageSystemMultiThreadedBindListeners, "AsyncMessagePassing.MultiThreaded.BindListeners", StressTestFlags)
	bool TMessageSystemMultiThreadedBindListeners::RunTest(const FString& InParameters)
	{
		TSharedPtr<FTestMessageSystem, ESPMode::ThreadSafe> TestSystem = FAsyncMessageSystemBase::CreateMessageSystem<FTestMessageSystem>();
		TestTrue(TEXT("MessageSystem startup has been called"), TestSystem->HasStarted());

		// A functor to add a listener which will be run as a task
		struct FAddListenerFunctor
		{
			TSharedPtr<FTestMessageSystem> MessageSys;

			TSharedPtr<FTestListener> ListenerObj;
			
			void operator()()
			{
				check(MessageSys.IsValid());
				
				// Personally I am expecting this to fail until I lock up the listener map with a CS
				const FAsyncMessageHandle Handle = MessageSys->BindListener(TestMessageId_A, ListenerObj.ToWeakPtr(), &FTestListener::Callback_Message_A);
				check(Handle.IsValid());
			}
		};
		
		TArray<TSharedPtr<FTestListener>> Listeners;
		
		TArray<UE::Tasks::FTask> PendingTasks;
		constexpr int32 NumTasksToSpawn = 5000;
		for (int32 i = 0; i < NumTasksToSpawn; ++i)
		{
			TSharedPtr<FTestListener> Listener = MakeShared<FTestListener>();
			Listeners.Emplace(Listener);
			
			UE::Tasks::FTask T = UE::Tasks::Launch(UE_SOURCE_LOCATION, FAddListenerFunctor { .MessageSys = TestSystem, .ListenerObj = Listener });
			PendingTasks.Emplace(T);
		}
		
		// Wait for all the tasks which queue messages to complete
		UE::Tasks::Wait(PendingTasks);

		TestSystem->RunOnce();
		
		TestSystem->Shutdown();
		TestSystem.Reset();
		
		return true;
	}

	// A test which will spin up several tasks on different threads to attempt to bind listeners to messages
	// on each thread. This will test that we can have different listeners binding on different threads
	// for the same message system. The test system's handle ID is an atomic<uint32>, so we should have no issue
	// generating handles from a bunch of different threads.
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(TMessageSystemMultiThreadedHandleGeneration, "AsyncMessagePassing.MultiThreaded.HandleGeneration", QuickTestFlags)
	bool TMessageSystemMultiThreadedHandleGeneration::RunTest(const FString& InParameters)
	{
		TSharedPtr<FTestMessageSystem, ESPMode::ThreadSafe> TestSystem = FAsyncMessageSystemBase::CreateMessageSystem<FTestMessageSystem>();
		TestTrue(TEXT("MessageSystem startup has been called"), TestSystem->HasStarted());

		// A functor to add a listener which will be run as a task
		struct FGenerateHandleFunctor
		{
			TSharedPtr<FTestMessageSystem> MessageSys;
			
			void operator()()
			{
				check(MessageSys.IsValid());
				
				// This should be valid because it is using atomics
				const FAsyncMessageHandle Handle = MessageSys->GenerateTestHandle();
			}
		};
		
		TArray<UE::Tasks::FTask> PendingTasks;
		constexpr int32 NumTasksToSpawn = 5000;
		for (int32 i = 0; i < NumTasksToSpawn; ++i)
		{
			UE::Tasks::FTask T = UE::Tasks::Launch(UE_SOURCE_LOCATION, FGenerateHandleFunctor { .MessageSys = TestSystem });
			PendingTasks.Emplace(T);
		}

		// Wait for all the tasks which queue messages to complete
		UE::Tasks::Wait(PendingTasks);

		TestSystem->RunOnce();
		
		TestSystem->Shutdown();
		TestSystem.Reset();
		
		return true;
	}

	// Test to queue messages from multiple threads on a single message system
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(TMessageSystemMultiThreadedSequenceIdTest, "AsyncMessagePassing.MultiThreaded.UniqueSequenceId", StressTestFlags)
	bool TMessageSystemMultiThreadedSequenceIdTest::RunTest(const FString& InParameters)
	{
		TSharedPtr<FTestMessageSystem, ESPMode::ThreadSafe> TestSystem = FAsyncMessageSystemBase::CreateMessageSystem<FTestMessageSystem>();
		TestTrue(TEXT("MessageSystem startup has been called"), TestSystem->HasStarted());

		TSet<uint32> UsedSequenceIds;
		uint32 NumTimesCalledback = 0;
		
		FAsyncMessageHandle ListenerHandle = TestSystem->BindListener(TestMessageId_A, [this, &NumTimesCalledback, &UsedSequenceIds](const FAsyncMessage& Message)
		{
			const uint32 SeqId = Message.GetSequenceId();
			const bool bIsUnique = !UsedSequenceIds.Contains(SeqId);
			TestTrue(TEXT("Has unique sequence ID"), bIsUnique);

			UsedSequenceIds.Add(SeqId);
			++NumTimesCalledback;
		});
		
		struct FQueueMessageFunctor
		{
			TWeakPtr<FTestMessageSystem> MessageSys;

			int32 AmountToAdd = 0;
			
			void operator()()
			{
				TSharedPtr<FTestMessageSystem> StrongMessageSys = MessageSys.Pin();
				check(StrongMessageSys.IsValid());

				// It is expected that this instanced struct payload data here would go out of scope.
				FTest_Payload_A PayloadData = {};
				PayloadData.IncrementAmount = AmountToAdd;
				FInstancedStruct PayloadDataInstance = FInstancedStruct::Make<FTest_Payload_A>(PayloadData);
				
				StrongMessageSys->QueueMessageForBroadcast(TestMessageId_A, PayloadDataInstance);
			}
		};

		TArray<UE::Tasks::FTask> PendingTasks;
		constexpr int32 NumTasksToSpawn = 3000;
		for (int32 i = 0; i < NumTasksToSpawn; ++i)
		{
			UE::Tasks::FTask T = UE::Tasks::Launch(UE_SOURCE_LOCATION, FQueueMessageFunctor { .MessageSys = TestSystem, .AmountToAdd = i });
			PendingTasks.Emplace(T);
		}

		// Wait for all the tasks which queue messages to complete
		UE::Tasks::Wait(PendingTasks);

		TestSystem->RunOnce();

		TestTrue(TEXT("Has the expected number of sequence IDs"),
			NumTimesCalledback == NumTasksToSpawn &&
			NumTasksToSpawn == UsedSequenceIds.Num());
		
		TestSystem->Shutdown();
		TestSystem.Reset();
		
		return true;
	}
};

#endif	// WITH_DEV_AUTOMATION_TESTS

// A test UFunction implmentation 
void UTestAsyncObject::CallbackFunction(const FAsyncMessage& Message)
{
	FConstStructView DataView = Message.GetPayloadView();
	const FTest_Payload_A* Data = Message.GetPayloadData<const FTest_Payload_A>();
	if (Data)
	{
		TestValue += Data->IncrementAmount;
	}
}
