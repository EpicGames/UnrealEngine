// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"	// For FString
#include "HAL/Platform.h"				// For int32 typedef
#include "Templates/SharedPointer.h"	// For TWeakPtr
#include "UObject/Object.h"				// For UObject

#include "AsyncMessageSystemTests.generated.h"

struct FAsyncMessage;
class FAsyncMessageSystemBase;

// Note: The payloads need to be USTRUCTs in order to test them with FInstancedStruct, which require ScriptStructs.

/**
 * A simple test payload type to ensure that the data of the payload is correct when you recieve a message.
 */
USTRUCT(NotBlueprintType, meta=(Hidden))
struct FTest_Payload_A
{
	GENERATED_BODY()

	UPROPERTY()
	int32 IncrementAmount = 78;

	UPROPERTY()
	FString SomeName = TEXT("Test string");

	UPROPERTY()
	float Bar = 123.0f;

	UPROPERTY()
	TObjectPtr<UObject> TestPointer = nullptr;
};

/**
 * A test payload to allow for us to check that you can bind listeners within the response
 * to another message, a "Nested" message binding.
 */
USTRUCT(NotBlueprintType, meta=(Hidden))
struct FNested_Payload
{
	GENERATED_BODY()
	
	TWeakPtr<FAsyncMessageSystemBase> MessageSystem = nullptr;
};

/**
 * A test UObject type which we can use to ensure that you can bind messages to UObjects.
 */
UCLASS(NotBlueprintable, NotBlueprintType)
class UTestAsyncObject : public UObject
{
	GENERATED_BODY()
public:
	void CallbackFunction(const FAsyncMessage& Message);

	UPROPERTY()
	int32 TestValue = 5;
};