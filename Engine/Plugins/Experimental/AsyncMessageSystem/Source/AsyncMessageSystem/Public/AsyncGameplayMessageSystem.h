// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphFwd.h"			// FGraphEventRef
#include "AsyncMessageSystemBase.h"
#include "Engine/EngineBaseTypes.h"		// For ELevelTick
#include "TaskSyncManager.h"

#define UE_API ASYNCMESSAGESYSTEM_API

class UWorld;

/**
 * Implementation of an async message system which schedules the processing of messages based on tick groups
 * and named thread async tasks, making it integrate nicely with common gameplay frameworks in Unreal.
 */
class FAsyncGameplayMessageSystem final :
	public FAsyncMessageSystemBase
{
	friend struct FMessageSystemTickFunction;

public:
	UE_API explicit FAsyncGameplayMessageSystem(UWorld* OwningWorld);

private:
	//~ Begin FAsyncMessageSystemBase interface
	virtual void Startup_Impl() override;
	virtual void Shutdown_Impl() override;
	virtual void PostQueueMessage(const FAsyncMessageId MessageId, const TArray<FAsyncMessageBindingOptions>& OptionsBoundTo) override;
	//~ End FAsyncMessageSystemBase interface

	/**
	 * Creates and registers tick functions for this message system to the outer world.
	 */
	void CreateTickFunctions();

	/**
	 * Destroy any registered tick functions on this message system
	 */
	void DestroyTickFunctions();

	/**
	 * Starts an async task to process the messages with listeners that have the given binding options.
	 * This should ONLY be called for binding options which are not tick groups.
	 * 
	 * @param Options The options of which to start async processing for.
	 */
	void StartAsyncProcessForBinding(const FAsyncMessageBindingOptions& Options);

	/**
	 * Called when a message is queued with listeners who have bindings options set to use a Tick Group.
	 *
	 * This will request a tick function to execute once via the TaskSyncManager to process the message queue for the given group.
	 * 
	 * @param Options The binding options that we want to enable the tick function for to process its listeners
	 */
	void EnableExecutionForTickGroupBinding(const FAsyncMessageBindingOptions& Options);

	/**
	 * Get or create the work handle required to request work from the task sync manager.
	 * 
	 * @param Group The tick group to get the sync work handle for
	 * @return Pointer to the work handle which can then be used to request work. Nullptr if the work handle cannot be created. 
	 */
	UE::Tick::FActiveSyncWorkHandle* GetOrCreateWorkHandleForTickGroup(const ETickingGroup Group);
	
	/**
	 * Processes all pending messages in the queue for the given tick group. This is meant to be called from a FTickFunction.
	 *
	 * This will just create the necessary FAsyncMessageBindingOptions args to call the FAsyncMessageSystemBase::ProcessMessageQueueForBinding.
	 */
	void ProcessMessagesForTickGroup(const ETickingGroup TickGroup);

	/**
	 * The world in which this message system belongs to. The tick functions of this message system will
	 * be registered to this world's persistent level.
	 */
	TWeakObjectPtr<UWorld> OuterWorld = nullptr;

	/**
	 * Tick functions which are created when this message system starts. There is one tick functions
	 * for each tick group that this message system supports and they will depend on one another
	 * to keep a deterministic ordering of ticking on this message system.
	 */
	TMap<ETickingGroup, TSharedPtr<FTickFunction>> TickFunctions;

	/**
	 * Handles that we can use to request work (in our case, process messages for a specific tick group)
	 *
	 * These handles need to be accessed behind the WorkHandleCreationCS
	 */
	TMap<ETickingGroup, UE::Tick::FActiveSyncWorkHandle> WorkHandles;

	/***
	 * The tick group earliest in the frame which this message system supports.
	 */
	static constexpr ETickingGroup EarliestSupportedTickGroup = TG_PrePhysics;
	
	/**
	 * The tick group that is the latest in frame time which this message system will support
	 */
	static constexpr ETickingGroup LatestSupportedTickGroup = TG_PostUpdateWork;

	/**
	 * Keep track of the state of what tick group is currently executing and what the last one was.
	 */
	ETickingGroup CurrentTickGroup = EarliestSupportedTickGroup;
	ETickingGroup LastTickedGroup = LatestSupportedTickGroup;
	
	/**  */
	FCriticalSection WorkHandleCreationCS;
};

#undef UE_API
