// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AsyncMessageId.h"
#include "HAL/Platform.h"				// For uint64 type declaration
#include "StructUtils/StructView.h"
#include "Templates/SharedPointer.h"	// For TWeakPtr

#include "AsyncMessage.generated.h"

#define UE_API ASYNCMESSAGESYSTEM_API

// A pre-processor define which will allow you to log the callstack of messages when they are queued.
// This can be helpful when debugging a build to see what messages are being queued and from where,
// since it can be hard to determine that information from the listener.
//
// By default, this will be enabled in the editor and non-shipping bulids.
//
// If you would like to enable this in a shipping build, then add a define to your game's .build.cs file:
//		PublicDefinitions.Add("ENABLE_ASYNC_MESSAGES_DEBUG=1");
//
// It is not recommended that you log out on every message, because it will add significant performance cost.
#ifndef ENABLE_ASYNC_MESSAGES_DEBUG
	#define ENABLE_ASYNC_MESSAGES_DEBUG WITH_EDITOR || !UE_BUILD_SHIPPING
#endif

class FAsyncMessageBindingEndpoint;

/**
 * A single async message which has been queued for broadcasting, and listeners are
 * interested in receiving the data from.
 *
 * Async Message Messages contain some data about when, where, and how the message was queued
 * which make it possible to synchronize game state across multiple threads with messages. 
 */
USTRUCT(BlueprintType)
struct FAsyncMessage final
{
	GENERATED_BODY()
public:
	
	FAsyncMessage() = default;
	
	UE_API FAsyncMessage(
		const FAsyncMessageId& MessageId,
		const FAsyncMessageId& MessageSourceId,
		const double MessageTimestamp,
		const uint64 CurrentFrame,
		const uint32 ThreadQueuedFrom,
		const uint32 MessageSequenceCounter,
		FConstStructView InPayloadData,
		TWeakPtr<FAsyncMessageBindingEndpoint> InBindingEndpoint);

	/**
	 * Returns this message's unique ID which listeners use to bind to it. 
	 */
	UE_API FAsyncMessageId GetMessageId() const;

	/**
	 * Sets the current message id which this FAsyncMessage represents to another message
	 * 
	 * @param NewMessageId The new message Id to set
	 */
	UE_API void SetMessageId(const FAsyncMessageId& NewMessageId);

	/**
	 * Returns this the source message which caused this message to be queued
	 */
	UE_API FAsyncMessageId GetMessageSourceId() const;
	
	/**
	 * Returns the timestamp which this message message was queued on.
	 */
	UE_API double GetQueueTimestamp() const;

	/**
	 * Returns the frame (GFrameNumber) of which this message was queued for broadcast.
	 */
	UE_API uint64 GetQueueFrame() const;

	/**
	 * Returns the thread Id from which this message was queued from ( FPlatformTLS::GetCurrentThreadId() )
	 */
	UE_API uint32 GetThreadQueuedFromThreadId() const;
	
	/**
	 * Returns the “sequence ID” of this message.
	 * If there are multiple of the same message queued for broadcast on a single frame,
	 * then this sequence ID can be used to order them and determine which was queued first if desired.	 
	 */
	UE_API uint32 GetSequenceId() const;

	/**
	 * Get the mutable payload data associated with this async message.
	 */
	UE_API FStructView GetPayloadView();

	/**
	 * Get the const payload data associated with this async message.
	 */
	UE_API FConstStructView GetPayloadView() const;

	/**
	 * Returns the typed data pointer of this message's payload. Nullptr if the payload is invalid.
	 */
	template<typename TDataType>
	TDataType* GetPayloadData() const
	{
		return PayloadCopy.GetPtr<TDataType>();
	}

	/**
	 * Returns the endpoint which this message should be broadcast on.
	 */
	UE_API TSharedPtr<FAsyncMessageBindingEndpoint> GetBindingEndpoint() const;
	
	inline void AddReferencedObjects(FReferenceCollector& Collector)
	{
		PayloadCopy.AddStructReferencedObjects(Collector);
	}

private:
	
	/**
	 * The Async message Id which this message is a part of. 
	 */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category="Async Message", meta = (AllowPrivateAccess = "true"))
	FAsyncMessageId MessageId = FAsyncMessageId::Invalid;

	/**
	 * The source of this message. This might be different from the "MessageId", because this could be a parent message which
	 * is being triggered by a child message. In that case, the "MessageId" would be the parent message, and the "MessageSourceId" would be the
	 * actual child message which had triggered.
	 *
	 * For example, if "message.colors.red" is a child message of "message.colors", then the "MessageId" would be "message.colors" and
	 * the MessageSourceId would be "message.colors.red".
	 *
	 * If the MessageSourceId and MessageId are the same value, then that means that this is the childmost message.
	 */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category="Async Message", meta = (AllowPrivateAccess = "true"))
	FAsyncMessageId MessageSourceId = FAsyncMessageId::Invalid;

	/**
	 * The time at which this message was queued ( FApp::GetCurrentTime() )
	 */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category="Async Message", meta = (AllowPrivateAccess = "true"))
	double QueueTime = 0.0;

	/**
	 * The frame at which this message was queued ( GFrameCounter )
	 */
	uint64 QueueFrame = 0;

	/**
	 * The ThreadId from which this message was queued from ( FPlatformTLS::GetCurrentThreadId() )
	 */
	uint32 ThreadQueuedFrom = 0;

	/**
	 * The sequence of this specific message if there were multiple in the message queue at one time.
	 * 
	 * This will allow async listeners to determine which message of this type was queued first if
	 * there are multiple.
	 */
	uint32 SequenceId = 0;

	/**
	 * A copy of the payload data that is created upon construction of this message.
	 * 
	 * This data is copied in the constructor of this message when the message is queued
	 * in order to make the data thread safe to access.
	 *
	 * If you would like to have more control over the payload data and how it is synconized
	 * (for example if you have a costly payload type which would be too expensive to copy)
	 * then you could make your payload struct contain a simple pointer and handle synchronization
	 * yourself with critical sections. 
	 * 
	 * TODO: When we copy the payload data, have it use a custom linear allocator on the owning message
	 * system to make it faster to copy and more cache friendly to the message system to iterate messages.
	 */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category="Async Message", meta = (DisplayName="Payload", AllowPrivateAccess = "true"))
	FInstancedStruct PayloadCopy;

	/**
	 * Pointer to which endpoint this message should be broadcast on.
	 * 
	 * Store this as a weak pointer because we don't want any messages to be keeping handlers
	 * from GC'ing if they are destroyed over time
	 */
	TWeakPtr<FAsyncMessageBindingEndpoint> BindingEndpoint = nullptr;
	
	// TODO: We could perhaps define a "lifetime" concept on these messages, similar to how AnimNext's
	// TraitEvents (TraitEvent.h) work, as well as consumption perhaps?

	/////////////////////////////////////////////////////////////////////
	// Debugging helpers
	
#if ENABLE_ASYNC_MESSAGES_DEBUG
public:
	
	struct FMessageDebugData
	{
		/**
		 * The native stack back trace at the time of this message being queued.
		 * Populated by FPlatformStackWalk::CaptureStackBackTrace in FAsyncMessageSystemBase::QueueMessageForBroadcast_Impl.
		 */
		TArray<uint64> NativeCallstack;
	
		/**
		 * A debugging nicety. This is a string which holds the callstack from where this message was queued from.
		 * Mutable so that we can cache the value in GetOrCreateNativeCallstackAsString.
		 */
		mutable FString NativeCallstackAsString;

		/**
		 * A debugging nicety. This is a string which holds the blueprint/script callstack from where this message was queued from.
		 */
		FString BlueprintScriptCallstack;

		/**
		* Unique int id assigned at the time of creation. Each message gets a unique ID which can then be used to look up
		* the debug data on the message store. 
		*/
		uint32 MessageId = 0u;

		/**
		 * Generates and caches the FString version of the platform callstack
		 * @return the Cachced string which represents the callstack.
		 */
		const FString& GetOrCreateNativeCallstackAsString() const;
	};
	
	UE_API const FString& GetNativeCallstack() const;

	UE_API const FString& GetBlueprintScriptCallstack() const;

	UE_API const uint32 GetDebugMessageId() const;

	UE_API void SetDebugData(FMessageDebugData* InData);
	
private:

	FMessageDebugData* DebugData = nullptr;
#endif	// #if ENABLE_ASYNC_MESSAGES_DEBUG
};

#undef UE_API
