// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "Components/ActorComponent.h"
#include "Templates/SharedPointer.h"

#include "AsyncMessageBindingComponent.generated.h"

class FAsyncMessageBindingEndpoint;

UINTERFACE(MinimalAPI, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UAsyncMessageBindingEndpointInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface to implement which will allow users to specify which binding endpoint
 * should be used in any script functions for the async message system.
 */
class IAsyncMessageBindingEndpointInterface
{
	GENERATED_BODY()

public:
	
	/**
	 * @return Pointer to the specific binding endpoint which you would like to set for your message or listener.
	 */
	virtual TSharedPtr<FAsyncMessageBindingEndpoint> GetEndpoint() const
	{
		return nullptr;
	}
};

/**
 * A blueprint component which will allow you to specify a specific endpoint to use
 * when queueing messages or binding listeners to the Async Message System.
 *
 * By default, this will create an endpoint on InitializeComponent (making it ready for use during Begin Play) 
 * and destroy it on UninitializeComponent.
 */
UCLASS(MinimalAPI, BlueprintType, HideCategories=(Object,Replication,Navigation), Meta=(BlueprintSpawnableComponent))
class UAsyncMessageBindingComponent :
	public UActorComponent,
	public IAsyncMessageBindingEndpointInterface
{
	GENERATED_BODY()

protected:

	//~ Begin UActorComponent interface
	virtual void InitializeComponent() override;
	virtual void UninitializeComponent() override;
	//~ end UActorComponent interface

	//~ Begin IAsyncMessageBindingEndpointInterface
	virtual TSharedPtr<FAsyncMessageBindingEndpoint> GetEndpoint() const override;
	//~ End IAsyncMessageBindingEndpointInterface

	ASYNCMESSAGESYSTEM_API virtual void CreateEndpoint();
	ASYNCMESSAGESYSTEM_API virtual void CleanupEndpoint();
	
	TSharedPtr<FAsyncMessageBindingEndpoint> Endpoint = nullptr;
};
