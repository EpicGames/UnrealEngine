// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AsyncMessage.h"
#include "AsyncMessageBindingOptions.h"
#include "AsyncMessageHandle.h"
#include "AsyncMessageId.h"
#include "Containers/Map.h"
#include "Templates/SharedPointer.h"

#define UE_API ASYNCMESSAGESYSTEM_API

/**
* The message binding endpoint is what stores the actual map of listeners and what Message Id's
* they are bound to.
*
* When binding to a message or queueing a message for broadcast, you can specify the endpoint.
* Listeners will only receive the messages queued for their own endpoints. This makes it easy to
* filter messages to a specific endpoint.
*
* These endpoints can be utilized within the game frame in a rather customizable way.
* One example would be to add create an Actor Component which has an endpoint. This way,
* you could send messages directly to a single specific actor.
*/
class FAsyncMessageBindingEndpoint final :
	public TSharedFromThis<FAsyncMessageBindingEndpoint>
{
public:

	FAsyncMessageBindingEndpoint() = default;
	
	using FMessageCallbackFunc = TFunction<void(const FAsyncMessage&)>;
	
	/**
	 * Data stored about a single individual listener of a message
	 */
	template <typename CallbackFuncSignature>
	struct FAsyncMessageIndividualListener
	{
		/**
		* The handle to this individual listener assigned when it is bound to this message system.
		*/
		FAsyncMessageHandle Handle;
		
		/**
		 * The delegate that listeners can bind to if they want to be called when this message
		 * is broadcast. 
		 */
		CallbackFuncSignature Callback;
	};
	
	/**
	 * Data store for everything related to a single message
	 */
	struct FAsyncMessageBoundData
	{
		/**
		 * A map of listeners for this message, ordered by their binding options.
		 * 
		 * This lets us keep track of how many binders are listening for this message on a specific
		 * tick group or thread id
		 */
		TMap<FAsyncMessageBindingOptions, TArray<FAsyncMessageIndividualListener<FMessageCallbackFunc>>> ListenerMap;
	};

	/**
	 * Returns data about listeners bound to the given async message id.
	 * 
	 * @param MessageId The message id to find bindings for.
	 * @return A pointer to the bound message data. Nullptr if there are no listeners bound to this message. 
	 */
	UE_API FAsyncMessageBoundData* GetBoundDataForMessage(const FAsyncMessageId& MessageId);

	/**
	 * Finds or adds a binding to the given message id.
	 * @param MessageId 
	 * @return 
	 */
	UE_API FAsyncMessageBoundData& FindOrAddMessageData(const FAsyncMessageId& MessageId);
	
	/**
	 * @return The total number of bound listeners to this endpoint
	 */
	UE_API uint32 GetNumberOfBoundListeners() const;

	/**
	 * @param Handle The message handle to check if it is bound to this endpoint
	 * @return True if the given handle is bound to this endpoint
	 */
	UE_API bool IsHandleBound(const FAsyncMessageHandle& Handle) const;

private:

	/**
	 * Map of message ID's to their associated message data.
	 */
	TMap<FAsyncMessageId, FAsyncMessageBindingEndpoint::FAsyncMessageBoundData> BoundMessageListenerMap;
};

#undef UE_API
