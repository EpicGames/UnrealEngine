// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"		// For ENamedThreads
#include "Engine/EngineBaseTypes.h"			// For ETickGroup
#include "Tasks/TaskPrivate.h"				// For ETaskPriority/EExtendedTaskPriority

#define UE_API ASYNCMESSAGESYSTEM_API

/**
 * Options used to specify when a binder would like to receive a message when it is broadcast.
 *
 * If you would like your listener to receive its messages within a specific tick group on the game thread,
 * create the options with a TickGroup and it will set UseTickGroup.
 * 
 * If you would like your listener to receive its messages within a specific named thread and/or pool of named threads 
 * create the options with a named thread (including advanced variants) and it will set UseNamedThreads.
 *
 * If you would like your listener to receive its messages from within a specifc set of UE::Tasks priority thread pools
 * create the options with a task priority and extended priority and it will set UseTaskPriorities.
 */
struct FAsyncMessageBindingOptions final
{
	// Construct from different options, these will call the appropriate Set function

	UE_API FAsyncMessageBindingOptions();
	UE_API FAsyncMessageBindingOptions(const ETickingGroup DesiredTickGroup);
	UE_API FAsyncMessageBindingOptions(const ENamedThreads::Type NamedThreads);
	UE_API FAsyncMessageBindingOptions(const UE::Tasks::ETaskPriority InTaskPriority, const UE::Tasks::EExtendedTaskPriority InExtendedTaskPriority);

	/**
	 * Determines which type of configuration this listener follows
	 */
	enum class EBindingType : uint8
	{
		/**
		 * If set, then this binding will receive a message on the main game thread during the specific tick group.
		 */
		UseTickGroup = 0,

		/**
		 * If set, then this binding will receive a message on a thread matching these named thread options.
		 */
		UseNamedThreads = 1,

		/**
		 * If set, then this binding will receive a message on a thread determined by the UE::Tasks::ETaskPriority
		 * and UE::Tasks::EExtendedTaskPriority.
		 */
		UseTaskPriorities = 2,
	};

	/** Determines which type of configuration this listener follows */
	EBindingType GetType() const
	{
		return Type;
	}


	/** Set this binding to receive a message on the main game thread during the specific tick group */
	UE_API void SetTickGroup(const ETickingGroup DesiredTickGroup);

	/** If type is UseTickGroup, this binding will receive a message on the main game thread during the specific tick group */
	UE_API ETickingGroup GetTickGroup() const;


	/** Set this binding to receive a message on a set of named threads */
	UE_API void SetNamedThreads(const ENamedThreads::Type NamedThreads);

	/** If type is UseNamedThreads, this will be used to determine which set of named threads you would like to use */
	UE_API ENamedThreads::Type GetNamedThreads() const;


	/** Set this binding to receive a message on a thread with the matching task priority and extended priority */
	UE_API void SetTaskPriorities(const UE::Tasks::ETaskPriority InTaskPriority, const UE::Tasks::EExtendedTaskPriority InExtendedTaskPriority = UE::Tasks::EExtendedTaskPriority::None);

	/** If type is UseTaskPriorities, this will be used to determine the task priority you would like to use */
	UE_API UE::Tasks::ETaskPriority GetTaskPriority() const;

	/** If type is UseTaskPriorities, this will be used to determine the extended task priority you would like to use */
	UE_API UE::Tasks::EExtendedTaskPriority GetExtendedTaskPriority() const;


private:

	/**
	 * The type of binding options which should be used to determine where a listener would like
	 * to receive its messages. 
	 */
	EBindingType Type = EBindingType::UseTickGroup;

	union
	{	
		/**
		 * Based on type, this identifies the thread identifier in which the binding object would like to receive a callback, or the tick group on game thread
		 */
		int32 ThreadOrGroup;

		struct 
		{
			/**
			 * Identifies the UE::Tasks task priority which this binding would like to receive a callback on
			 */
			UE::Tasks::ETaskPriority TaskPriority;

			/**
			 * Identifies the UE::Tasks extended priority which this binding would like to receive a callback on
			 */
			UE::Tasks::EExtendedTaskPriority ExtendedTaskPriority;
		};
	};

public:

	bool operator==(const FAsyncMessageBindingOptions& Other) const
	{
		return
			Type == Other.Type &&
			ThreadOrGroup == Other.ThreadOrGroup;
	}

	bool operator!=(const FAsyncMessageBindingOptions& Other) const
	{
		return !(*this == Other);
	}

	friend inline uint32 GetTypeHash(const FAsyncMessageBindingOptions& Opts)
	{
		// Overflow is fine here, just need a semi-unique integer
		return static_cast<uint32>(Opts.ThreadOrGroup) + static_cast<uint8>(Opts.Type);
	}
};

static_assert(sizeof(FAsyncMessageBindingOptions) == sizeof(uint64));

#undef UE_API
