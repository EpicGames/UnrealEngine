// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AsyncMessageId.h"
#include "Containers/UnrealString.h"	// For FString
#include "HAL/Platform.h"				// For uint32 type declaration
#include "Templates/SharedPointer.h"	// For TWeakPtr

#include "AsyncMessageHandle.generated.h"

#define UE_API ASYNCMESSAGESYSTEM_API

class FAsyncMessageBindingEndpoint;

/**
 * Handle used to identify a bound listener to an Async Message.
 * These handles are unique to each bound listener and created by the
 * owning FAsyncMessageSystemBase that the Message was bound to.
 */
USTRUCT(BlueprintType)
struct FAsyncMessageHandle final
{
	GENERATED_BODY()
public:

	friend class FAsyncMessageSystemBase;

	/**
	 * Default constructor of a Message handle. This default handle will be invalid.
	 * Valid handles should only be created from an operating async message system (@see FAsyncMessageSystemBase).
	 */
	FAsyncMessageHandle() = default;

	/**
	 * Returns true if this Message handle has a valid internal ID.
	 */
	UE_API bool IsValid() const;

	/**
	 * Returns the internal ID of this handle.
	 */
	UE_API uint32 GetId() const;

	/**
	 * Returns the Message in which this handle is bound to.
	 */
	UE_API FAsyncMessageId GetBoundMessageId() const;

	/**
	 * Returns a formatted string which represents this handle's data.
	 */
	UE_API FString ToString() const;

	UE_API bool operator==(const FAsyncMessageHandle& Other) const;
	UE_API bool operator!=(const FAsyncMessageHandle& Other) const;
	UE_API bool operator>=(const FAsyncMessageHandle& Other) const;
	UE_API bool operator<(const FAsyncMessageHandle& Other) const;
	ASYNCMESSAGESYSTEM_API friend uint32 GetTypeHash(const FAsyncMessageHandle& InMapping);

	/**
	 * @return A pointer to the endpoint which this handle is bound to.
	 */
	UE_API TSharedPtr<FAsyncMessageBindingEndpoint> GetBindingEndpoint() const;
	
	/**
	 * Represents an invalid handle which cannot have any Message listeners bound to it
	 * and means that no more Messages will be received. 
	 */
	static UE_API const FAsyncMessageHandle Invalid;

private:

	/**
	 * Private constructor for use by the message system
	 */
	UE_API FAsyncMessageHandle(
		const uint32 InHandleValue,
		 const FAsyncMessageId BoundMessageId,
		TWeakPtr<FAsyncMessageBindingEndpoint> InBindingEndpoint);

	/**
	 * Internal index for message handles which we consider to be invalid.
	 */
	static constexpr uint32 InvalidHandleIndex = 0u;
	
	/**
	 * The value of this internal handle.
	 */
	UPROPERTY(VisibleAnywhere, Category="Async Message")
	uint32 InternalHandle = FAsyncMessageHandle::InvalidHandleIndex;

	/**
	 * The async message that this handle is bound to.
	 */
	UPROPERTY(VisibleAnywhere, Category="Async Message")
	FAsyncMessageId BoundMessage = FAsyncMessageId::Invalid;

	/**
	 * The endpoint which this handle is bound to.
	 */
	TWeakPtr<FAsyncMessageBindingEndpoint> BindingEndpoint = nullptr;
};

#undef UE_API
