// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"	// For FString
#include "GameplayTagContainer.h"		// For FGameplayTag
#include "UObject/NameTypes.h"			// For FName

#include "AsyncMessageId.generated.h"

#define UE_API ASYNCMESSAGESYSTEM_API

/**
 * Represents a single message which can be bound to and queued in the async message system.
 *
 * This ID is what you use to bind to (listen for) messages which are being broadcast, as well as
 * actually queue the message for broadcasting.
 *
 * Internally this is simply a FGameplayTag, which is how this Message's parent hierarchy is determined.  
 */
USTRUCT(BlueprintType)
struct FAsyncMessageId final
{
	GENERATED_BODY()

public:
	FAsyncMessageId() = default;

	/**
	 * Constructs a MessageId from the given FName.
	 *
	 * Note: This will internally be requesting a gameplay tag representation of this FName, and will ensure if it is not found.
	 * 
	 * @param MessageName The FName representation of this MessageId
	 */
	UE_API FAsyncMessageId(const FName MessageName);
	
	/**
	 * Constructs a FAsyncMessageId from the FName associated with the given Gameplay Tag. 
	 * 
	 * @param MessageTag The gameplay tag to construct a Message Id from
	 */
	UE_API FAsyncMessageId(const FGameplayTag& MessageTag);
	~FAsyncMessageId() = default;

	/**
	 * @return True if this is a valid MessageId which can be used to bind and queue messages.
	 */
	[[nodiscard]] UE_API bool IsValid() const;

	/**
	 * @return The raw FName which represents this MessageId 
	 */
	[[nodiscard]] UE_API FName GetMessageName() const;

	/**
	 * @return FString representation of this MessageId 
	 */
	[[nodiscard]] UE_API FString ToString() const;

	/**
	* Returns the parent of this async message. The parent is parsed out of the underlying FName which represents this message by '.' delimeters.
	* 
	* For example, the parent of "message.id.test" is "message.id" , and the parent of "message.id" is "message". If there are no '.' delimeters in the
	* message's name then it does not have a parent, and this will return FAsyncMessageId::Invalid
	*/
	[[nodiscard]] UE_API FAsyncMessageId GetParentMessageId() const;	

	bool operator==(const FAsyncMessageId& Other) const
	{
		return InternalMessageTag == Other.InternalMessageTag;
	}

	bool operator!=(const FAsyncMessageId& Other) const
	{
		return !(*this == Other);
	}

	friend inline uint32 GetTypeHash(const FAsyncMessageId& InMessageId)
	{
		// Just use the internal FName for the hash type
		return GetTypeHash(InMessageId.InternalMessageTag);
	}

	/**
	 * Represents an invalid message ID which cannot be bound to.
	 */
	static UE_API const FAsyncMessageId Invalid;

	/**
	 * Walks up the parent heirarchy of the given Startingmessage. This will iterate up the parent message chain
	 * and calling the given ForEachMessageFunc for it so long as it is valid.
	 *
	 * This does include the given starting message!
	 */
	static UE_API void WalkMessageHierarchy(const FAsyncMessageId StartingMessage, TFunctionRef<void(const FAsyncMessageId MessageId)> ForEachMessageFunc);
	
protected:
	
	/**
	 * The FName which represents this async message ID
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Messages")
	FGameplayTag InternalMessageTag;
};


#undef UE_API
