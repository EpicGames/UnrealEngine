// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AsyncMessage.h"
#include "AsyncMessageBindingOptions.h"
#include "AsyncMessageId.h"
#include "Containers/Map.h"
#include "Containers/Queue.h"
#include "Templates/SharedPointer.h"

/**
 * Implementation for the actual storage of async messages for different binding methods.
 * 
 * This will store a message queue for each type of supports Binding option for listeners,
 * allowing listeners to be in control over when they receive messages from the queue (what tick group,
 * thread ID, etc).
 */
class FAsyncMessageStore final
{
public:
	FAsyncMessageStore() = default;
	~FAsyncMessageStore() = default;

	UE_NONCOPYABLE(FAsyncMessageStore);

public:

	uint32 EnqueueMessage(FAsyncMessage&& MessageToQueue, OUT TArray<FAsyncMessageBindingOptions>& BindingTypesAddedTo);

	[[nodiscard]] TOptional<FAsyncMessage> GetNextMessageForBindingOption(const FAsyncMessageBindingOptions& Binding);

	void AddMessageToBinding(const FAsyncMessageId& MessageId, const FAsyncMessageBindingOptions& Binding);

	/**
	 * Removes the given message ID from the specified message binding. This is called when an message handler is unbound
	 * 
	 */
	bool RemoveMessageFromBinding(const FAsyncMessageId& MessageId, const FAsyncMessageBindingOptions& Binding);

	/**
	 * Adds references to objects stored in the message queue to the GC collection
	 * This will include the FInstancedStruct payload data on the message queue
	 * to prevent objects from being destroyed if they are referenced in a message paylod.
	 */
	void AddReferencedObjects(FReferenceCollector& Collector);
	
protected:

	/**
	* CS for when access to the MessageBindingQueues is required 
	* to enqueue or dequeue messages accordingly.
	*/
	mutable FCriticalSection MessageQueueCriticalSection;
	
	// The message queue for a single set of binding options
	struct FBindingOptionsMessageQueue
	{
		FBindingOptionsMessageQueue()
		{
			MessageQueue = MakeShared<TArray<FAsyncMessage>>();
		}

		~FBindingOptionsMessageQueue()
		{
			MessageQueue.Reset();
		}
		
		// Keep track of what messages are bound within this interface so that we can quickly
		// find which queues require which messages.
		// The value of this map is how many times it has been bound.
		TMap<FAsyncMessageId, uint32> MessagesWithTheseBindings;
		
		// The queue of messages for this binding option		
		TSharedPtr<TArray<FAsyncMessage>> MessageQueue;
	};

	/**
	 * Tracks the binding options to their associated message queues.
	 * 
	 * Each binding option needs its own message queue so that we can enable
	 * the listeners of the messages to control _when_ they receive an message during the frame. 
	 */
	TMap<FAsyncMessageBindingOptions, FBindingOptionsMessageQueue> MessageBindingQueues;

#if ENABLE_ASYNC_MESSAGES_DEBUG
	/**
	 * Unique ID assigned to a message if we are recording its queue callstack.
	 * 0 would be an invalid message id.
	 */
	std::atomic<uint32> NextDebugMessageId = 1u;

	/**
	 * Creates and adds a debug messsage to the MessageDebugCallstacks map.
	 * Records the current native and BP/script callstacks 
	 * @return Debug message Id which can be used to look it up in the MessageDebugCallstacks map.
	 */
	uint32 CreateDebugMessageData();

	/**
	 * A map of DebugMessageId's to their associated debug data.
	 * The uint32 in the TTuple is the number of remaining message queues to be processed
	 * for that message. It is decremented every time the message queue is processed, and when
	 * it reaches zero is removed from this map.
	 */
	TMap<uint32, TTuple<uint32, FAsyncMessage::FMessageDebugData>> MessageDebugCallstacks;
#endif
};
