// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AsyncMessage.h"
#include "AsyncMessageHandle.h"
#include "AsyncMessageBindingEndpoint.h"
#include "AsyncMessageId.h"
#include "AsyncMessageStore.h"
#include <atomic>
#include "Containers/MpscQueue.h"
#include "StructUtils/StructView.h"
#include "Templates/SharedPointer.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API ASYNCMESSAGESYSTEM_API

class FAsyncMessageBindingEndpoint;

/**
 * Base abstract implementation of an async message system which can be used to easily
 * pass signals and messages across different threads in Unreal.
 */
class FAsyncMessageSystemBase :
	public TSharedFromThis<FAsyncMessageSystemBase>
{
public:

	/**
	 * Creates a new shared pointer to a message system of type TMessageSystemType
	 * and calls the "Startup" function on it.
	 *
	 * Make sure to call the "Shutdown" function on the message system prior to its destruction.
	 */
	template<class TMessageSystemType, typename... TArgs>
	[[nodiscard]] static inline TSharedPtr<TMessageSystemType, ESPMode::ThreadSafe> CreateMessageSystem(TArgs&&... Args);

	using FMessageCallbackFunc = TFunction<void(const FAsyncMessage&)>;
	
protected:
	
	// We have a protected constructor on the message system because you should always use the
	// "CreateMessageSystem" function above to make one. This allows us to ensure the virtual
	// "Startup" function is called.
	FAsyncMessageSystemBase() = default;

public:

	UE_NONCOPYABLE(FAsyncMessageSystemBase)
	UE_API virtual ~FAsyncMessageSystemBase();

	/**
	 * Binds the given callback to this message so that when an message of type MessageId is broadcast, it will be executed.
	 *
	 * @param MessageId		The async message to bind to
	 * @param Callback		The callback which should be executed when 'MessageId' is broadcast.
	 * @param Options		Options describing when the listener would like to receive this callback (what tick group or named async thread)
	 *
	 * @return Async Message handle which can be used to unbind this listener. If the listener is not unbound, then a weak reference to it will be checked
	 * and cleaned up when this message system is shut down. If the listener goes out of scope and the callback function is no longer valid, it will be unbound.
	 */
	UE_API FAsyncMessageHandle BindListener(
		const FAsyncMessageId MessageId,
		FMessageCallbackFunc&& Callback,
		const FAsyncMessageBindingOptions& Options = {},
		TWeakPtr<FAsyncMessageBindingEndpoint> BindingEndpoint = nullptr);

	/**
	 * Binds the given callback to this message so that when an message of type MessageId is broadcast, it will be executed.
	 *
	 * @param MessageId			The async message to bind to
	 * @param WeakOwnerObject	The object that owns the bound listener callback. If this object is not valid at the time this message is called, then nothing will happen
	 *							(the Callback delegate will not be execute on it)
	 *							
	 * @param Callback			The callback which should be executed when 'MessageId' is broadcast.
	 * @param Options		Options describing when the listener would like to receive this callback (what tick group or named async thread)
	 * 
	 * @return Async Message handle which can be used to unbind this listener. If the listener is not unbound, then a weak reference to it will be checked
	 * and cleaned up when this message system is shut down. If the listener goes out of scope and the callback function is no longer valid, it will be unbound.
	 */
	template <typename TOwner = UObject>
	FAsyncMessageHandle BindListener(
		const FAsyncMessageId MessageId,
		TWeakObjectPtr<TOwner> WeakOwnerObject,
		void(TOwner::* Callback)(const FAsyncMessage&),
		const FAsyncMessageBindingOptions& Options = {},
		TWeakPtr<FAsyncMessageBindingEndpoint> BindingEndpoint = nullptr);
	
	/**
	 * Binds the given callback to this message so that when an message of type MessageId is broadcast, it will be executed.
	 *
	 * @param MessageId		The async message to bind to
	 * @param WeakObject	The object that owns the bound listener callback. If this object is not valid at the time this message is called, then nothing will happen
	 *						(the Callback delegate will not be execute on it)
	 * 
	 * @param Callback		The callback which should be executed when 'MessageId' is broadcast.
	 * @param Options		Options describing when the listener would like to receive this callback (what tick group or named async thread)
	 *
	 * @return Async message handle which can be used to unbind this listener. If the listener is not unbound, then a weak reference to it will be checked
	 * and cleaned up when this message system is shut down. If the listener goes out of scope and the callback function is no longer valid, it will be unbound.
	 */
	template <typename TOwner>
	FAsyncMessageHandle BindListener(
		const FAsyncMessageId MessageId,
		TWeakPtr<TOwner> WeakObject,
		void(TOwner::* Callback)(const FAsyncMessage&),
		const FAsyncMessageBindingOptions& Options = {},
		TWeakPtr<FAsyncMessageBindingEndpoint> BindingEndpoint = nullptr);

	/**
	 * Unbinds the given listener from its message so that it will no longer receive callbacks
	 * when that message is broadcast.
	 *
	 * Listeners will be marked for unbinding and removed from the listener map the next time
	 * that this system is processed.
	 */
	UE_API void UnbindListener(const FAsyncMessageHandle& HandleToUnbind);	
	
	/**
	 * Queues the given async message for broadcast the next time that this
	 * message system processes its message queue
	 *
	 * @param MessageId		The message that you would like to queue for broadcasting
	 * @param PayloadData	Pointer to the payload data of this message. This payload data will be COPIED to the message queue to
	 *						to make it safe for listeners on other threads. 
	 *
	 * @return True if this message had an listeners bound to it and it was successfully queued
	 */
	UE_API bool QueueMessageForBroadcast(
		const FAsyncMessageId MessageId,
		FConstStructView PayloadData = nullptr,
		TWeakPtr<FAsyncMessageBindingEndpoint> BindingEndpoint = nullptr);

	/**
	 * Processes all async message on within the queue for the given binding options.
	 * 
	 * @param Options Which async message binding queue to process the messages from.
	 */
	UE_API void ProcessMessageQueueForBinding(const FAsyncMessageBindingOptions& Options);

protected:
	
	/**
	 * Provides an opportunity for this message system to run any startup logic which it may need to.
	 * Utilize this to start any tick groups or other async background tasks which may need to occur.
	 */
	UE_API void Startup();

public:
	
	/**
	 * Manual shutdown of this message system. Once this is called, no more listeners can be bound and no more
	 * messages can be queued. Use this to clean up any background work which may need to be done that was created in
	 * Startup.
	 */
	UE_API void Shutdown();

	UE_API void AddReferencedObjects(class UObject* InReferencer, FReferenceCollector& Collector);

protected:

	/**
	* Provides an opportunity to run any startup logic that subclasses of this Base message
	* system may need, such as creating tick groups for specific binding options.
	*/
	virtual void Startup_Impl() = 0;

	/**
	* Allows you to clean up anything you may need to with which was created during startup.
	*/
	virtual void Shutdown_Impl() = 0;

	/**
	* Handles binding the message callback to the given async message handle. This will return true if the binding was successful 
	* and false if it failed for some reason (the message system could be shutting down, the MessageId could be invalid, etc.
	*/
	[[nodiscard]] UE_API virtual bool BindListener_Impl(
		const FAsyncMessageHandle& HandleToBindTo,
		const FAsyncMessageId MessageId,
		FMessageCallbackFunc&& Callback,
		const FAsyncMessageBindingOptions& Options);
	
	/**
	 * Marks the given handle as ready to be unbound. Enqueues this handle to a queue which will be 
	 * processed on the next call to ProcessUnbindHandleRequests.
	 */
	UE_API virtual void UnbindListener_Impl(const FAsyncMessageHandle& HandleToUnbind);

	/**
	* Actually queues the message for broadcasting.
	* 
	* Note: The payload data provided will be copied to make it safe for listeners on different threads. 
	*/
	UE_API virtual bool QueueMessageForBroadcast_Impl(
		const FAsyncMessageId MessageId,
		FConstStructView PayloadData,
		TWeakPtr<FAsyncMessageBindingEndpoint> BindingEndpoint);
	
	/**
	 * Called after an message was queued (from QueueMessageForBroadcast_Impl). This forces child classes of the message system
	 * to handle the queuing of messages and kick of some scheduling for actually processing the queues accordingly.
	 *
	 * @param MessageId			The message which was queued
	 * @param OptionsBoundTo	All of the binding options that this message was queued for.
	 */
	virtual void PostQueueMessage(const FAsyncMessageId MessageId, const TArray<FAsyncMessageBindingOptions>& OptionsBoundTo) = 0;

	/**
	 * Actual implementation of how we process the message queue. This will dequeue all messages out fo the message store
	 * for the given binding type, and broadcast their callbacks for each message and it's heirarhcy. 
	 * 
	 * @param Options The binding options which we are currently interested in processing messages for 
	 */
	UE_API virtual void ProcessMessageQueueForBinding_Impl(const FAsyncMessageBindingOptions& Options);
	
	/**
	 * Allow an opportunity to pre-process any messages or bindings which may have occurred before 
	 * the actual processing of the message queue
	 */
	UE_API virtual void PreProcessMessagesQueue(const FAsyncMessageBindingOptions& Options);

	/**
	 * Unbind any handles which have been flagged for removal in "UnbindListener"
	 */
	UE_API void ProcessUnbindHandleRequests();

	/**
	 * Iterate through the PendingBoundListenerQueue and bind the callback functions as necessary.
	 * We need to defer the adding of listeners in order to support binding new listeners in response to
	 * an message which is currently being processed (otherwise we would be modifying the listener array as we iterate it)
	 */
	UE_API void ProcessListenersPendingBinding();
	
	/**
	 * Generates a new async message handle by atomically incrementing the NextMessageHandleId.
	 * If the new value of NextMessageHandleId is above the max allowed handle id, it will be wrapped.
	 */
	[[nodiscard]] UE_API FAsyncMessageHandle GenerateNextValidMessageHandle(
		const FAsyncMessageId ForMessageId,
		TWeakPtr<FAsyncMessageBindingEndpoint> Endpoint);
	
	/**
	 * Critical section for when the message listeners map is changed (listeners are bound/unbound)
	 */
	mutable FCriticalSection MessageListenerMapCS;
	
	/**
	 * The message handle which we should use to create next
	 * New handles can be created any time that we bind a listener, which can be on any thread
	 */
	std::atomic<uint32> NextMessageHandleId = FAsyncMessageHandle::InvalidHandleIndex + 1u;

	/**
	 * The message sequence count. Each time an message is queued, we will increment
	 * this. Doing this will allow you to determine if a message was queued for broadcast
	 * before or after another of the same type or binding option, which can be useful if you are receiving
	 * the same message multiple times on a single frame.
	 *
	 * This is reset back to zero every time we process a binding in ProcessMessageQueueForBinding_Impl. 
	 */
	std::atomic<uint32> NextMessageSequence = 0;

	/**
	 * A quick flag to check if this system is in the process of shutting down.
	 */
	std::atomic<bool> bIsShuttingDown = false;

	/**
	 * The default binding endpoint for this message system. When listeners are bound or messages are queued
	 * which do not specify a specific endpoint, this one will be used.
	 */
	TSharedPtr<FAsyncMessageBindingEndpoint> DefaultBindingEndpoint = nullptr;

	/**
	 * Where we store the messages when they are queued for broadcasting and where we can
	 * process individual sets of FAsyncMessageBindingOptions for their messages 
	 */
	FAsyncMessageStore MessageStore;
	
	/**
	* Stores data about a listener which has been bound but cannot be added at the time of which
	* FAsyncMessageSystemBase::BindListener_Impl would be called, and needs to be deferred.
	*/
	template <typename TCallbackFuncSignature>
	struct FPendingBoundListener
	{
		// The message which this listener desires to be bound to
		const FAsyncMessageId MessageId;

		// The data for this message, populated in FAsyncMessageSystemBase::BindListener_Impl
		FAsyncMessageBindingEndpoint::FAsyncMessageIndividualListener<TCallbackFuncSignature> Data;

		// Binding options that this listener requires
		FAsyncMessageBindingOptions Options = {};
	};
	
	/**
	 * A queue of listeners which are pending to be added. Listeners are added to this
	 * queue during FAsyncMessageSystemBase::BindListener_Impl.
	 * 
	 * We do this because otherwise, we would be modifying the array of listeners whilst
	 * iterating it, which would cause an error as well as some undefined behavior if you bind
	 * another object to the same message. 
	 */
	TMpscQueue<FPendingBoundListener<FMessageCallbackFunc>> PendingBoundListenerQueue;

	/**
	 * message handles that are currently queued for removal. Handles get added to this queue
	 * by UnbindListener_Impl, and the queue is processed in PreProcessMessagesQueue
	 */
	TMpscQueue<FAsyncMessageHandle> MessageHandlesPendingRemoval;		

	/**
	* A set of handles which have been marked for removal, but their associated listeners have not been
	* fully bound yet (their binding endpoint does not have any data about them yet). 
	* 
	* Handles in this set will be skipped if they are seen during ProcessListenersPendingBinding, as they 
	* have already been marked for removal.
	* 
	* This should only be utilized behind the MessageListenerMapCS scope lock
	*/
	TSet<FAsyncMessageHandle> IncompleteBoundHandlesToRemove;

#if WITH_DEV_AUTOMATION_TESTS
public:
    /**
     * A test-only helper function to quickly create an message handle from an index.
     */
    [[nodiscard]] static UE_API FAsyncMessageHandle GenerateHandleAtIndex(
    	const uint32 Index,
    	const FAsyncMessageId& ForId,
    	TWeakPtr<FAsyncMessageBindingEndpoint> Endpoint);
#endif
};

//////////////////////////////////////////////////////////////
// Template Definitions
//////////////////////////////////////////////////////////////

template <class TMessageSystemType, typename ... TArgs>
TSharedPtr<TMessageSystemType, ESPMode::ThreadSafe> FAsyncMessageSystemBase::CreateMessageSystem(TArgs&&... Args)
{
	// Only accept subclasses of FAsyncMessageSystemBase here
	static_assert(std::is_base_of<FAsyncMessageSystemBase, TMessageSystemType>::value, "TMessageSystemType not derived from FAsyncMessageSystemBase");
	
	TSharedPtr<TMessageSystemType> MessageSystem = MakeShared<TMessageSystemType>(Forward<TArgs>(Args)...);

	MessageSystem->Startup();
	
	return MessageSystem;
}

template <typename TOwner>
FAsyncMessageHandle FAsyncMessageSystemBase::BindListener(
	const FAsyncMessageId MessageId,
	TWeakObjectPtr<TOwner> WeakOwnerObject,
	void(TOwner::* Callback)(const FAsyncMessage&),
	const FAsyncMessageBindingOptions& Options,
	TWeakPtr<FAsyncMessageBindingEndpoint> BindingEndpoint /*= nullptr*/)
{
	// This function is only for UObjects, because TWeakObjectPtr is a thing
	static_assert(std::is_base_of<UObject, TOwner>::value, "TOwner not derived from UObject");

	const FAsyncMessageHandle NewListenerHandle = GenerateNextValidMessageHandle(MessageId, BindingEndpoint);
	
	const bool bWasSuccessfullyBound = BindListener_Impl(
		NewListenerHandle,
		MessageId,
		[WeakOwnerObject, Callback, NewListenerHandle, WeakThisPtr = this->AsWeak()](const FAsyncMessage& Payload)
		{
			if (TStrongObjectPtr<TOwner> StrongObj = WeakOwnerObject.Pin())
			{
				(StrongObj.Get()->*Callback)(Payload);
			}
			// Otherwise, if the owning object which we have bound has gone out of scope, unbind its handle
			else if (TSharedPtr<FAsyncMessageSystemBase> SharedPtr = WeakThisPtr.Pin())
			{
				SharedPtr->UnbindListener(NewListenerHandle);
			}
		},
		Options);

	return bWasSuccessfullyBound ? NewListenerHandle : FAsyncMessageHandle::Invalid;
}

template <typename TOwner>
FAsyncMessageHandle FAsyncMessageSystemBase::BindListener(
	const FAsyncMessageId MessageId,
	TWeakPtr<TOwner> WeakObject,
	void(TOwner::* Callback)(const FAsyncMessage&),
	const FAsyncMessageBindingOptions& Options,
	TWeakPtr<FAsyncMessageBindingEndpoint> BindingEndpoint /*= nullptr*/)
{
	const FAsyncMessageHandle NewListenerHandle = GenerateNextValidMessageHandle(MessageId, BindingEndpoint);
	
	const bool bWasSuccessfullyBound = BindListener_Impl(
		NewListenerHandle,
		MessageId,
		[WeakObject, Callback, NewListenerHandle, WeakThisPtr = this->AsWeak()](const FAsyncMessage& Payload)
		{
			if (TSharedPtr<TOwner> StrongObject = WeakObject.Pin())
			{
				if (TOwner* Obj = StrongObject.Get())
				{
					(Obj->*Callback)(Payload);
					return;	// return early because we had a valid object
				}
			}

			// The owning object pointer is invalid. Unbind it so that we don't attempt to call it again
			if (TSharedPtr<FAsyncMessageSystemBase> SharedPtr = WeakThisPtr.Pin())
			{
				SharedPtr->UnbindListener(NewListenerHandle);
			}
		},
		Options);

	return bWasSuccessfullyBound ? NewListenerHandle : FAsyncMessageHandle::Invalid;
}

#undef UE_API
