// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

ASYNCMESSAGESYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogAsyncMessageSystem, Log, All);