// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"		// For TMulticastDelegate
#include "Engine/World.h"
#include "Subsystems/WorldSubsystem.h"

#include "AsyncMessageWorldSubsystem.generated.h"

#define UE_API ASYNCMESSAGESYSTEM_API

class FAsyncMessageSystemBase;

/**
 * A world subsystem which will create a unique message system per-world
 * and allow for easy access to the message system interface for gameplay code.
 *
 * An example of using the message system like this is:
 *
 *		TSharedPtr<FAsyncMessageSystemBase> Sys = UAsyncMessageWorldSubsystem::GetSharedMessageSystem(GetWorld());
 *		Sys->QueueMessageForBroadcast(MessageToQueue, MessagePayloadToQueue);
 */
UCLASS(MinimalAPI, NotBlueprintType)
class UAsyncMessageWorldSubsystem :
	public UWorldSubsystem
{

GENERATED_BODY()

public:
	
	/**
	 * Static helper function to get the message system for a world.
	 * 
	 * @tparam TMessageSystemType	The type of message system to create, must be a subclass of FAsyncMessageSystemBase
	 * @param InWorld				The world to get the message system of
	 * @return						Shared pointer to the given world's message system. Null if the given world is invalid.
	 */
	template<class TMessageSystemType = FAsyncMessageSystemBase>
	static TSharedPtr<TMessageSystemType> GetSharedMessageSystem(const UWorld* InWorld);

	/**
	 * Returns the message system for this world subsystem.
	 * 
	 * @tparam TMessageSystemType	The type of message system to create, must be a subclass of FAsyncMessageSystemBase
	 * @return						Shared pointer to this world subsystem's current message system
	 */
	template<class TMessageSystemType = FAsyncMessageSystemBase>
	TSharedPtr<TMessageSystemType> GetSharedMessageSystem() const;

	/**
	 * Delegate which is broadcast when this subsystem's message system is shutdown. Can be used to clean up any bindings
	 * to this subsystem's message system when the world goes out of scope
	 */
	TMulticastDelegate<void()> OnShutdownMessageSystem;

	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	
protected:

	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;

	/**
	 * Creates the MessageSystem for this world subsystem. 
	 */
	UE_API void InitMessageSystem();

	/**
	* Shutdown and cleans up this worlds message system. 
	*/
	UE_API void ShutdownMessageSystem();
	
	/**
	 * Instance of the message system for this world subsystem. This shares the lifetime of this subsystem
	 * and will be released in ShutdownMessageSystem, upon Deinitialize of this subsystem.
	 */
	TSharedPtr<FAsyncMessageSystemBase, ESPMode::ThreadSafe> MessageSystem = nullptr;
};

//////////////////////////////////////////////////////////////
// Template Definitions
//////////////////////////////////////////////////////////////

template <class TMessageSystemType>
TSharedPtr<TMessageSystemType> UAsyncMessageWorldSubsystem::GetSharedMessageSystem(const UWorld* InWorld)
{
	static_assert(std::is_base_of<FAsyncMessageSystemBase, TMessageSystemType>::value, "TMessageSystemType not derived from FAsyncMessageSystemBase");

	if (InWorld)
	{
		if (const UAsyncMessageWorldSubsystem* Sys = InWorld->GetSubsystem<UAsyncMessageWorldSubsystem>())
		{
			return Sys->GetSharedMessageSystem<TMessageSystemType>();	
		}
	}

	return nullptr;
}

template <class TMessageSystemType>
TSharedPtr<TMessageSystemType> UAsyncMessageWorldSubsystem::GetSharedMessageSystem() const
{
	static_assert(std::is_base_of<FAsyncMessageSystemBase, TMessageSystemType>::value, "TMessageSystemType not derived from FAsyncMessageSystemBase");

	return StaticCastSharedPtr<TMessageSystemType>(MessageSystem);
}

#undef UE_API
